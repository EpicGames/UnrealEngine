{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Geometry Cache",
	"Description": "Support for distilled Geometry animations",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/AlembicImporter/#importasgeometrycache",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "GeometryCacheEd",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac"
			]
		},
		{
			"Name": "GeometryCacheSequencer",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac"
			]
		},
		{
			"Name": "GeometryCacheStreamer",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		},
		{
			"Name": "GeometryCache",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryCacheTracks",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}