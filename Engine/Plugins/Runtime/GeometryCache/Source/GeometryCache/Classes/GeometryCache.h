// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/Interface_AssetUserData.h"
#include "RenderCommandFence.h"
#include "UObject/ObjectSaveContext.h"

#include "GeometryCache.generated.h"

#define UE_API GEOMETRYCACHE_API

class UGeometryCacheTrack;
class UMaterialInterface;
struct FGeometryCacheMeshData;

DECLARE_LOG_CATEGORY_EXTERN(LogGeometryCache, Log, All);

/**
* A Geometry Cache is a piece/set of geometry that consists of individual Mesh/Transformation samples.
* In contrast with Static Meshes they can have their vertices animated in certain ways. * 
*/
UCLASS(MinimalAPI, hidecategories = Object, BlueprintType, config = Engine)
class UGeometryCache : public UObject, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()
public:
	//~ Begin UObject Interface.
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual FString GetDesc() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	/**
	* AddTrack
	*
	* @param Track - GeometryCacheTrack instance that is a part of the GeometryCacheAsset
	* @return void
	*/
	UE_API void AddTrack(UGeometryCacheTrack* Track);
		
	/** Clears all stored data so the reimporting step can fill the instance again*/
	UE_API void ClearForReimporting();	

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this Geometry cache object*/
	UPROPERTY(Category = ImportSettings, VisibleAnywhere, Instanced)
	TObjectPtr<class UAssetImportData> AssetImportData;	

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;
#endif
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache)
	TArray<TObjectPtr<UMaterialInterface>> Materials;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache)
	TArray<FName> MaterialSlotNames;

	/** GeometryCache track defining the samples/geometry data for this GeomCache instance */
	UPROPERTY(VisibleAnywhere, Category=GeometryCache)
	TArray<TObjectPtr<UGeometryCacheTrack>> Tracks;

	/** Set the start and end frames for the GeometryCache */
	UE_API void SetFrameStartEnd(int32 InStartFrame, int32 InEndFrame);

	/** Get the start frame */
	UE_API int32 GetStartFrame() const;

	/** Get the end frame */
	UE_API int32 GetEndFrame() const;
	
	/** Calculate it's duration */
	UE_API float CalculateDuration() const;

	/** Get the Frame at the Specified Time */
	UE_API int32  GetFrameAtTime(const float Time) const;

	/** Get the mesh data at the specified time */
	UE_API void GetMeshDataAtTime(float Time, TArray<FGeometryCacheMeshData>& OutMeshData) const;

	/** Get the hash of the meshes data of the GeometryCache */
	UE_API FString GetHash() const;

public:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Hidden)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	UE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	DECLARE_DELEGATE_OneParam(FOnPreSave, UGeometryCache*);
	FOnPreSave OnPreSave;

private:
	/** A fence which is used to keep track of the rendering thread releasing the geometry cache resources. */
	FRenderCommandFence ReleaseResourcesFence;

protected:
	UPROPERTY(BlueprintReadOnly, Category = GeometryCache)
	int32 StartFrame;

	UPROPERTY(BlueprintReadOnly, Category = GeometryCache)
	int32 EndFrame;

	UPROPERTY()
	uint64 Hash;
};

#undef UE_API
