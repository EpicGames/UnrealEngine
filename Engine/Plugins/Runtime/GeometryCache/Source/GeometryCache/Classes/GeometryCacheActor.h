// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "GeometryCacheActor.generated.h"

#define UE_API GEOMETRYCACHE_API

class UGeometryCacheComponent;

/** GeometryCache actor, serves as a place-able actor for GeometryCache objects*/
UCLASS(MinimalAPI, ComponentWrapperClass)
class AGeometryCacheActor : public AActor
{
	GENERATED_UCLASS_BODY()

	// Begin AActor overrides.
#if WITH_EDITOR
	 UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override; 
#endif // WITH_EDITOR
	// End AActor overrides.
private:
	UPROPERTY(Category = GeometryCacheActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|GeometryCache", AllowPrivateAccess = "true"))
	TObjectPtr<UGeometryCacheComponent> GeometryCacheComponent;
public:
	/** Returns GeometryCacheComponent subobject **/
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API UGeometryCacheComponent* GetGeometryCacheComponent() const;
};

#undef UE_API
