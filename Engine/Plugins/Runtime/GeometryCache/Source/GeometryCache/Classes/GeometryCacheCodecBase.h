// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Serialization/BulkData.h"

#include "GeometryCacheCodecBase.generated.h"

#define UE_API GEOMETRYCACHE_API

struct FResourceSizeEx;
struct FGeometryCacheMeshData;
class UGeometryCacheTrackStreamable;
class FGeometryCacheTrackStreamableRenderResource;
class FRHICommandListBase;

/**
The smallest unit of streamed geometry cache data.

An instance of this struct is kept resident for all chuncks only the BulkData member may not
be loaded yet. Other info is permanently available.
*/
struct FStreamedGeometryCacheChunk
{
	FStreamedGeometryCacheChunk();

	/** Size of the chunk of data in bytes */
	int32 DataSize;

	/** Frame index of the earliest frame stored in this block */
	float FirstFrame;

	/** End frame index of the interval this chunk contains data for.
	This closed so the last frame is included in the interval
	*/
	float LastFrame;

	/** Bulk data if stored in the package. */
	FByteBulkData BulkData;

	/** Serialization. */
	void Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex);

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);
};

/**
 * Arguments passed to the decoder. This allows adding arguments easily without needing to change all decoders.
 */
struct FGeometryCacheCodecDecodeArguments
{
	FGeometryCacheCodecDecodeArguments(FGeometryCacheTrackStreamableRenderResource &SetTrack, TArray<FStreamedGeometryCacheChunk> &SetChunks, uint32 SetFrameIdentifier, FGeometryCacheMeshData& SetOutMeshData)
		: Track(SetTrack),
		Chunks(SetChunks),
		FrameIdentifier(SetFrameIdentifier),
		OutMeshData(SetOutMeshData)
	{
		// We do common validation checking here so it doesn't have to be done in every codec individually
		checkf(static_cast<float>(FrameIdentifier) >= Chunks[0].FirstFrame, TEXT("Requested frame not in the range of the provided chunks, or chunks are not properly ordered"));
		checkf(static_cast<float>(FrameIdentifier) <= Chunks.Last().LastFrame, TEXT("Requested frame not in the range of the provided chunks, or chunks are not properly ordered"));
	}

	FGeometryCacheTrackStreamableRenderResource &Track;
	TArray<FStreamedGeometryCacheChunk> &Chunks;
	uint32 FrameIdentifier;
	FGeometryCacheMeshData& OutMeshData;
};

/**
	Render thread side state. This is not a full blown FRenderResource it is a child instead of the
	UGeometryCacheTrackStreamable's Render Resource instance.
*/
class FGeometryCacheCodecRenderStateBase
{
public:
	/**
		Called on the game thread as part of UGeometryCacheCodecBase::CreateRenderState
	*/
	FGeometryCacheCodecRenderStateBase(const TArray<int32> &SetTopologyRanges) : TopologyRanges(SetTopologyRanges)
	{
		check(IsInGameThread());
	}
	
	/** 
		This will be called on the render thread
	*/
	virtual ~FGeometryCacheCodecRenderStateBase() { check(IsInRenderingThread()); }

	/**
		Called once we are on the render thread this can create any render buffers etc.
	*/
	virtual void InitRHI(FRHICommandListBase&) {};

	virtual bool DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args) { return false; }
	
	virtual bool IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB);

private:
	TArray<int32> TopologyRanges;
};

struct FGeometryCacheCodecEncodeArguments
{
	FGeometryCacheCodecEncodeArguments(const FGeometryCacheMeshData& SetMeshData, float SetSampleTime, bool bSetSameTopologyAsPrevious)
		: MeshData(SetMeshData),
		SampleTime(SetSampleTime),
		bSameTopologyAsPrevious(bSetSameTopologyAsPrevious)
	{}

	const FGeometryCacheMeshData& MeshData; // Mesh data for the submitted sample
	float SampleTime; // Time for the submitted sample
	bool bSameTopologyAsPrevious; // Is the topology the same as the previously submitted sample
};


/** Interface for assets/objects that can own UserData **/
UCLASS(MinimalAPI, hidecategories = Object)
class UGeometryCacheCodecBase : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual ~UGeometryCacheCodecBase() = default;

	virtual bool DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args) { return false; }
	virtual bool DecodeBuffer(const uint8* Buffer, uint32 BufferSize, FGeometryCacheMeshData& OutMeshData) { return false; }

	/** Create a new FGeometryCacheCodecRenderStateBase for this codec. The returned object will be owned by the calling code. */
	virtual FGeometryCacheCodecRenderStateBase *CreateRenderState() { return nullptr; }

	// Encoding is only available in editor builds
#if WITH_EDITORONLY_DATA
	UE_API virtual void BeginCoding(TArray<FStreamedGeometryCacheChunk> &AppendChunksTo);
	virtual void EndCoding() {}
	UE_API virtual void CodeFrame(const FGeometryCacheCodecEncodeArguments& Args);
	virtual int32 GetEncoderFrameNumer() { return EncoderBaseData.FrameId; }
#endif

	static UE_API bool IsSameTopology(const TArray<int32>& TopologyRanges, int32 FrameA, int32 FrameB);
	virtual bool IsSameTopology(int32 FrameA, int32 FrameB) { return IsSameTopology(TopologyRanges, FrameA, FrameB); }

protected:
#if WITH_EDITORONLY_DATA
	struct FEncoderBaseData
	{
		int32 FrameId;
	};
	FEncoderBaseData EncoderBaseData;
#endif

	// Frames at which the topology of the decoded frames changes sorted in increasing order
	// this allows fast topology queries between arbitrary frames.
	// each codec has to fill this
	UPROPERTY(VisibleAnywhere, Category = GeometryCache)
	TArray<int32> TopologyRanges;
};

#undef UE_API
