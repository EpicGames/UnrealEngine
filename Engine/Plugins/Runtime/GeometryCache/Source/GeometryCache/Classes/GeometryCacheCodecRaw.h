// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCacheCodecBase.h"

#include "GeometryCacheCodecRaw.generated.h"

#define UE_API GEOMETRYCACHE_API

class UGeometryCacheTrackStreamable;

struct FGeometryCacheCodecRenderStateRaw : FGeometryCacheCodecRenderStateBase
{
	FGeometryCacheCodecRenderStateRaw(const TArray<int32> &SetTopologyRanges) : FGeometryCacheCodecRenderStateBase(SetTopologyRanges) {}
	virtual bool DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args) override;
};

UCLASS(MinimalAPI, hidecategories = Object)
class UGeometryCacheCodecRaw : public UGeometryCacheCodecBase
{
	GENERATED_UCLASS_BODY()

	UE_API virtual bool DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args) override;
	UE_API virtual FGeometryCacheCodecRenderStateBase *CreateRenderState() override;

#if WITH_EDITORONLY_DATA
	UE_API virtual void BeginCoding(TArray<FStreamedGeometryCacheChunk> &AppendChunksTo) override;
	UE_API virtual void EndCoding() override;
	UE_API virtual void CodeFrame(const FGeometryCacheCodecEncodeArguments& Args) override;
#endif

private:
	UPROPERTY(VisibleAnywhere, Category = GeometryCache)
	int32 DummyProperty;

#if WITH_EDITORONLY_DATA
	struct FEncoderData
	{
		int32 CurrentChunkId; // Index in the AppendChunksTo list of the chunk we are working on
		TArray<FStreamedGeometryCacheChunk> *AppendChunksTo; // Any chunks this codec creates will be appended to this list
	};
	FEncoderData EncoderData;
#endif
};

#undef UE_API
