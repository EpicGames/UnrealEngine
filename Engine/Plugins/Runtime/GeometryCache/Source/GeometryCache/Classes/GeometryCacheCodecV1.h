// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCacheCodecBase.h"

#include "GeometryCacheCodecV1.generated.h"

#define UE_API GEOMETRYCACHE_API

class UGeometryCacheTrackStreamable;
class ICodecDecoder;
class ICodecEncoder;

struct FGeometryCacheCodecRenderStateV1 : FGeometryCacheCodecRenderStateBase
{
	FGeometryCacheCodecRenderStateV1(const TArray<int32> &SetTopologyRanges); 
	~FGeometryCacheCodecRenderStateV1();	
	
	virtual bool DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args) override;
};

UCLASS(MinimalAPI, hidecategories = Object)
class UGeometryCacheCodecV1 : public UGeometryCacheCodecBase
{
	GENERATED_UCLASS_BODY()

	UE_API virtual ~UGeometryCacheCodecV1();

	UE_API virtual bool DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args) override;
	UE_API virtual bool DecodeBuffer(const uint8* Buffer, uint32 BufferSize, FGeometryCacheMeshData& OutMeshData) override;
	UE_API virtual FGeometryCacheCodecRenderStateBase *CreateRenderState() override;

#if WITH_EDITORONLY_DATA
	UE_API virtual void InitializeEncoder(float InVertexQuantizationPrecision, int32 InUVQuantizationBitRange);
	UE_API virtual void BeginCoding(TArray<FStreamedGeometryCacheChunk> &AppendChunksTo) override;
	UE_API virtual void EndCoding() override;
	UE_API virtual void CodeFrame(const FGeometryCacheCodecEncodeArguments& Args) override;	
#endif

private:
	ICodecDecoder* Decoder;

#if WITH_EDITORONLY_DATA
	ICodecEncoder* Encoder;

	struct FEncoderData
	{
		int32 CurrentChunkId; // Index in the AppendChunksTo list of the chunk we are working on
		TArray<FStreamedGeometryCacheChunk>* AppendChunksTo; // Any chunks this codec creates will be appended to this list
	};
	FEncoderData EncoderData;
	int32 NextContextId;
#endif
};

#undef UE_API
