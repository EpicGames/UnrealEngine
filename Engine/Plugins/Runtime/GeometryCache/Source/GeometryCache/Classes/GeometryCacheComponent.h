// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"

#include "GeometryCacheComponent.generated.h"

#define UE_API GEOMETRYCACHE_API

class UGeometryCache;
struct FGeometryCacheMeshData;

/** Stores the RenderData for each individual track */
USTRUCT()
struct FTrackRenderData
{
	GENERATED_USTRUCT_BODY()

	FTrackRenderData() : Matrix(FMatrix::Identity), BoundingBox(EForceInit::ForceInitToZero), MatrixSampleIndex(INDEX_NONE), BoundsSampleIndex(INDEX_NONE) {}

	/** Transform matrix used to render this specific track. 
		This goes from track local space to component local space.
	*/
	FMatrix Matrix;

	/** Bounding box of this specific track */
	FBox BoundingBox;

	/** Sample Id's of the values we currently have registered the component with. */
	int32 MatrixSampleIndex;
	int32 BoundsSampleIndex;
};

/** GeometryCacheComponent, encapsulates a GeometryCache asset instance and implements functionality for rendering/and playback of GeometryCaches */
UCLASS(MinimalAPI, ClassGroup = (Rendering), hidecategories = (Object, LOD), meta = (BlueprintSpawnableComponent))
class UGeometryCacheComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

	/** Required for access to (protected) TrackSections */
	friend class FGeometryCacheSceneProxy;
		
	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void FinishDestroy() override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UActorComponent Interface.
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	/** Update LocalBounds member from the local box of each section */
	UE_API void UpdateLocalBounds();
	//~ Begin USceneComponent Interface.	

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual int32 GetMaterialIndex(FName MaterialSlotName) const override;
	UE_API virtual TArray<FName> GetMaterialSlotNames() const override;
	UE_API virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const override;
	//~ End UMeshComponent Interface.

	/**
	* OnObjectReimported, Callback function to refresh section data and update scene proxy.
	*
	* @param ImportedGeometryCache
	*/
	UE_API void OnObjectReimported(UGeometryCache* ImportedGeometryCache);
	
public:
	/** Start playback of GeometryCache */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void Play();

	/** Start playback of GeometryCache from the start */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void PlayFromStart();

	/** Start playback of GeometryCache in reverse*/
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void PlayReversed();
	
	/** Start playback of GeometryCache from the end and play in reverse */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void PlayReversedFromEnd();

	/** Pause playback of GeometryCache */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void Pause();

	/** Stop playback of GeometryCache */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void Stop();

	/** Get whether this GeometryCache is playing or not. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool IsPlaying() const;

	/** Get whether this GeometryCache is playing in reverse or not. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool IsPlayingReversed() const;

	/** Get whether this GeometryCache is looping or not. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool IsLooping() const;

	/** Set whether this GeometryCache is looping or not. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetLooping( const bool bNewLooping);

	/** Get whether this GeometryCache is interpolating frames. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool IsInterpolatingFrames() const;

	/** Set whether this GeometryCache is interpolating frames. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetInterpolateFrames(const bool bNewInterpolating);

	/** Get whether this GeometryCache is extrapolating frames. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool IsExtrapolatingFrames() const;

	/** Set whether this GeometryCache is extrapolating frames. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetExtrapolateFrames(const bool bNewExtrapolating);

	/** Get current playback speed for GeometryCache. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetPlaybackSpeed() const;

	/** Set new playback speed for GeometryCache. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetPlaybackSpeed(const float NewPlaybackSpeed);

	/** Get the motion vector scale. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetMotionVectorScale() const;

	/** Set new motion vector scale. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetMotionVectorScale(const float NewMotionVectorScale);

	/** Change the Geometry Cache used by this instance. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool SetGeometryCache( UGeometryCache* NewGeomCache );
	
	/** Getter for Geometry cache instance referred by the component
		Note: This getter is not exposed to blueprints as you can use the readonly Uproperty for that
	*/
	UE_API UGeometryCache* GetGeometryCache() const;

	/** Get current start time offset for GeometryCache. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetStartTimeOffset() const;

	/** Set current start time offset for GeometryCache. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetStartTimeOffset(const float NewStartTimeOffset);

	/** Get the current animation time for GeometryCache. Includes the influence of elapsed time and SetStartTimeOffset */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetAnimationTime() const;

	/** Get the current elapsed time for GeometryCache. Doesn't include the influence of StartTimeOffset */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetElapsedTime() const;

	/** Get the playback direction for GeometryCache. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetPlaybackDirection() const;

	/** Geometry Cache instance referenced by the component */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GeometryCache)
	TObjectPtr<UGeometryCache> GeometryCache;
		
	/** Get the duration of the playback */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API float GetDuration() const;

	/** Get the number of frames */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API int32 GetNumberOfFrames() const;

	/** Get the number of tracks */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API int32 GetNumberOfTracks() const;

	/** Override wireframe color? */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetOverrideWireframeColor(bool bOverride);

	/** Check whether we are overriding the wireframe color or not. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API bool GetOverrideWireframeColor() const;

	/** Set the color, used when overriding the wireframe color is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void SetWireframeOverrideColor(const FLinearColor Color);

	/** Get the wireframe override color, used when overriding the wireframe color is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API FLinearColor GetWireframeOverrideColor() const;

public:
	/** Helper to get the frame of the ABC asset at time provided*/
	UE_API int32 GetFrameAtTime(const float Time) const;

	/** Helper to get the time at this frame */
	UE_API float GetTimeAtFrame(const int32 Frame) const;

	/** Helper to make the animation jump to this time*/
	UE_API void SetCurrentTime(const float Time);

public:
	/** Functions to override the default TickComponent */
	UE_API void SetManualTick(bool bInManualTick);
	UE_API bool GetManualTick() const;

	UE_API void ResetAnimationTime();

	UFUNCTION(BlueprintCallable, Category = "Components|GeometryCache")
	UE_API void TickAtThisTime(const float Time, bool bInIsRunning, bool bInBackwards, bool bInIsLooping);

#if WITH_EDITOR
	// Animation Helpers for Transport Controls
	UE_API void StepForward();

	UE_API void ForwardEnd();

	UE_API void StepBackward();

	UE_API void BackwardEnd();

	UE_API void ToggleLooping();

	UE_API TArray<FString> GetTrackNames() const;
#endif

protected:
	/**
	* Invalidate both the Matrix and Mesh sample indices
	*/
	UE_API void InvalidateTrackSampleIndices();

	/**
	* ReleaseResources, clears and removes data stored/copied from GeometryCache instance	
	*/
	UE_API void ReleaseResources();

	/**
	* Updates the game thread state of a track section
	*
	* @param TrackIndex - Index of the track we want to update
	*/
	UE_API bool UpdateTrackSection(int32 TrackIndex, float Time);

	/**
	* CreateTrackSection, Create/replace a track section.
	*
	* @param TrackIndex - Index of the track to create (corresponds to an index on the geometry cache)
	*/
	UE_API void CreateTrackSection(int32 TrackIndex);

	/**
	* SetupTrackData
	* Call CreateTrackSection for all tracks in the GeometryCache assigned to this object.
	*/
	UE_API void SetupTrackData();

	/**
	* ClearTrackData
	* Clean up data that was required for playback of geometry cache tracks
	*/
	UE_API void ClearTrackData();

	/**
	 * Jumps animation to time specified.
	 */
	UE_API void JumpAnimationToTime(float Time, bool bInIsRunning, bool bInBackwards, bool bInIsLooping);

	/**
	 * Helper method to tick animation by one frame in PlayDirection
	 */
	UE_API void StepAnimationFrame(bool bInBackwards);

protected:
	UPROPERTY(EditAnywhere, Interp, Category = GeometryCache)
	bool bRunning;

	UPROPERTY(EditAnywhere, Interp, Category = GeometryCache)
	bool bLooping;

	/** Enable frame interpolation for sub-frame sampling of constant topologies */
	UPROPERTY(EditAnywhere, Category = GeometryCache, AdvancedDisplay)
	bool bInterpolateFrames;

	/** Enable frame extrapolation for sub-frame sampling of non-constant topologies with imported motion vectors */
	UPROPERTY(EditAnywhere, Category = GeometryCache, AdvancedDisplay)
	bool bExtrapolateFrames;

	UPROPERTY(EditAnywhere, Interp, Category = GeometryCache, meta = (UIMin = "-14400.0", UIMax = "14400.0", ClampMin = "-14400.0", ClampMax = "14400.0"))
	float StartTimeOffset;

	UPROPERTY(EditAnywhere, Interp, Category = GeometryCache, meta = (UIMin = "0.0", UIMax = "4.0", ClampMin = "0.0", ClampMax = "512.0"))
	float PlaybackSpeed;

	/** Scale factor to apply to the imported motion vectors */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GeometryCache, meta = (ClampMin = "0.0", ClampMax = "100", UIMin = "0.0", UIMax = "100"))
	float MotionVectorScale;

	UPROPERTY(VisibleAnywhere, Category = GeometryCache)
	int32 NumTracks;

	UPROPERTY(VisibleAnywhere, transient, Category = GeometryCache)
	float ElapsedTime;

	/** Component local space bounds of geometry cache mesh */
	FBoxSphereBounds LocalBounds;

	/** Array containing the TrackData (used for rendering) for each individual track*/
	TArray<FTrackRenderData> TrackSections;

	/** Play (time) direction, either -1.0f or 1.0f */
	float PlayDirection;

	/** Duration of the animation (maximum time) */
	UPROPERTY(BlueprintReadOnly, Category = GeometryCache)
	float Duration;

	UPROPERTY(EditAnywhere, Category = GeometryCache)
	bool bManualTick;

	/** Do we override the wireframe rendering color? */
	UPROPERTY(EditAnywhere, Category = GeometryCache)
	bool bOverrideWireframeColor;

	/** The wireframe override color. */
	UPROPERTY(EditAnywhere, Category = GeometryCache, meta = (EditCondition = "bOVerrideWireframeColor"))
	FLinearColor WireframeOverrideColor;
};

#undef UE_API
