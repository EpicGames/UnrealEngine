// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Box.h"
#include "PackedNormal.h"
#include "GeometryCacheMeshData.generated.h"

#define UE_API GEOMETRYCACHE_API

/** Stores per-batch data used for rendering */
USTRUCT()
struct FGeometryCacheMeshBatchInfo
{
	GENERATED_USTRUCT_BODY()

	/** Starting index into IndexBuffer to draw from */
	uint32 StartIndex;
	/** Total number of Triangles to draw */
	uint32 NumTriangles;
	/** Index to Material used to draw this batch */
	uint32 MaterialIndex;

	friend FArchive& operator<<(FArchive& Ar, FGeometryCacheMeshBatchInfo& Mesh)
	{
		Ar << Mesh.StartIndex;
		Ar << Mesh.NumTriangles;
		Ar << Mesh.MaterialIndex;
		// Empty batches?!?
		check(Mesh.NumTriangles > 0);
		return Ar;
	}	
};

/** Stores info on the attributes of a vertex in a mesh */
USTRUCT()
struct FGeometryCacheVertexInfo
{
	GENERATED_USTRUCT_BODY()

	/** Info on which attributes are present or valid */
	bool bHasTangentX;
	bool bHasTangentZ;
	bool bHasUV0;
	bool bHasColor0;
	bool bHasMotionVectors;
	bool bHasImportedVertexNumbers;

	bool bConstantUV0;
	bool bConstantColor0;
	bool bConstantIndices;

	FGeometryCacheVertexInfo()
	{
		bHasTangentX = false;
		bHasTangentZ = false;
		bHasUV0 = false;
		bHasColor0 = false;
		bHasMotionVectors = false;
		bHasImportedVertexNumbers = false;
		bConstantUV0 = false;
		bConstantColor0 = false;
		bConstantIndices = false;
	}

	FGeometryCacheVertexInfo(
		bool bSetTangentX,
		bool bSetTangentZ,
		bool bSetUV0,
		bool bSetColor0,
		bool bSetMotionVectors0
	) : bHasTangentX(bSetTangentX),
		bHasTangentZ(bSetTangentZ),
		bHasUV0(bSetUV0),
		bHasColor0(bSetColor0),
		bHasMotionVectors(bSetMotionVectors0)
	{}

	friend FArchive& operator<<(FArchive& Ar, FGeometryCacheVertexInfo& Mesh)
	{
		Ar << Mesh.bHasTangentX;
		Ar << Mesh.bHasTangentZ;
		Ar << Mesh.bHasUV0;
		Ar << Mesh.bHasColor0;
		Ar << Mesh.bHasMotionVectors;
		Ar << Mesh.bHasImportedVertexNumbers;

		Ar << Mesh.bConstantUV0;
		Ar << Mesh.bConstantColor0;
		Ar << Mesh.bConstantIndices;
		
		return Ar;
	}
};

/** Stores per Track/Mesh data used for rendering*/
USTRUCT()
struct FGeometryCacheMeshData
{
	GENERATED_USTRUCT_BODY()

	FGeometryCacheMeshData()
	: BoundingBox(EForceInit::ForceInitToZero)
	, Hash(0)
	{}
	
	/** Draw-able vertex data */
	TArray<FVector3f> Positions;
	TArray<FVector2f> TextureCoordinates;
	TArray<FPackedNormal> TangentsX;
	TArray<FPackedNormal> TangentsZ;
	TArray<FColor> Colors;

	/** The original imported vertex numbers, as they come from the DCC. One for each vertex position. */
	TArray<uint32> ImportedVertexNumbers;

	/** Motion vector for each vertex. The number of motion vectors should be zero (= no motion vectors) or identical to the number of vertices. */
	TArray<FVector3f> MotionVectors;
	/** Array of per-batch info structs*/
	TArray<FGeometryCacheMeshBatchInfo> BatchesInfo;
	/** Bounding box for this sample in the track */
	FBox3f BoundingBox;
	/** Indices for this sample, used for drawing the mesh */
	TArray<uint32> Indices;	
	/** Info on the vertex attributes */
	FGeometryCacheVertexInfo VertexInfo;

	/** A reusable array of bytes used to decode frame data. */
	TArray<uint8> TempBytes;
		
	/** Serialization for FVertexAnimationSample. */
	friend GEOMETRYCACHE_API FArchive& operator<<(FArchive& Ar, FGeometryCacheMeshData& Mesh);

	/** Serialization for const FVertexAnimationSample. */
	friend FArchive& operator<<(FArchive& Ar, const FGeometryCacheMeshData& Mesh)
	{
		check(Ar.IsSaving());
		return (Ar << const_cast<FGeometryCacheMeshData&>(Mesh));
	}

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
	{
		// Calculate resource size according to what is actually serialized
		CumulativeResourceSize.AddUnknownMemoryBytes(Positions.Num() * sizeof(FVector3f));
		CumulativeResourceSize.AddUnknownMemoryBytes(TextureCoordinates.Num() * sizeof(FVector2f));
		CumulativeResourceSize.AddUnknownMemoryBytes(TangentsX.Num() * sizeof(FPackedNormal));
		CumulativeResourceSize.AddUnknownMemoryBytes(TangentsZ.Num() * sizeof(FPackedNormal));
		CumulativeResourceSize.AddUnknownMemoryBytes(Colors.Num() * sizeof(FColor));

		CumulativeResourceSize.AddUnknownMemoryBytes(MotionVectors.Num() * sizeof(FVector3f));
		CumulativeResourceSize.AddUnknownMemoryBytes(BatchesInfo.Num() * sizeof(FGeometryCacheMeshBatchInfo));

		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(Positions));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(TextureCoordinates));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(TangentsX));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(TangentsZ));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(Colors));

		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(BatchesInfo));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(BoundingBox));
		CumulativeResourceSize.AddUnknownMemoryBytes(Indices.Num() * sizeof(uint32));
		CumulativeResourceSize.AddUnknownMemoryBytes(ImportedVertexNumbers.Num() * sizeof(uint32));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(Indices));
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(VertexInfo));
	}

	/** Return a hash of the content of the GeometryCacheMeshData */
	UE_API uint64 GetHash() const;

private:
	mutable uint64 Hash;
};

#undef UE_API
