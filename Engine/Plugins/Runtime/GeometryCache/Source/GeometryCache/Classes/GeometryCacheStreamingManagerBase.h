// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ContentStreaming.h"

class UGeometryCacheComponent;
class UGeometryCacheTrackStreamable;

/**
* Note IStreamingManager is not really anything like an interface it contains code and members and whatnot.
* So we just play along here to keep the spirit of the existing audio and texture streaming managers.
*/
struct IGeometryCacheStreamingManager : public IStreamingManager
{
	IGeometryCacheStreamingManager() {}

	/** Virtual destructor */
	virtual ~IGeometryCacheStreamingManager() = default;

	/** Getter of the singleton */
	GEOMETRYCACHE_API static struct IGeometryCacheStreamingManager& Get();

	/** Adds a new cache to the streaming manager. */
	virtual void AddGeometryCache(UGeometryCacheTrackStreamable* Cache) = 0;

	/** Removes a cache from the streaming manager. */
	virtual void RemoveGeometryCache(UGeometryCacheTrackStreamable* Cache) = 0;

	/** Returns true if this is a cache is managed by the streaming manager. */
	virtual bool IsManagedGeometryCache(const UGeometryCacheTrackStreamable* Cache) const = 0;

	/** Returns true if this Sound Wave is currently streaming a chunk. */
	virtual bool IsStreamingInProgress(const UGeometryCacheTrackStreamable* Cache) = 0;

	virtual bool CanCreateSoundSource(const FWaveInstance* WaveInstance) const = 0;

	/** Adds a new Sound Source to the streaming manager. */
	virtual void AddStreamingComponent(UGeometryCacheComponent* CacheComponent) = 0;

	/** Removes a Sound Source from the streaming manager. */
	virtual void RemoveStreamingComponent(UGeometryCacheComponent* CacheComponent) = 0;

	/** Returns true if this is a streaming Sound Source that is managed by the streaming manager. */
	virtual bool IsManagedComponent(const UGeometryCacheComponent* CacheComponent) const = 0;

	/**
	* Gets a pointer to a chunk of cahed geometry data
	*
	* @param Track Animation track we want a chunk from
	* @param ChunkIndex	Index of the chunk we want
	* @return Either the desired chunk or NULL if it's not loaded
	*/
	virtual const uint8* GetLoadedChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex, uint32* OutChunkSize = NULL) const = 0;

};
