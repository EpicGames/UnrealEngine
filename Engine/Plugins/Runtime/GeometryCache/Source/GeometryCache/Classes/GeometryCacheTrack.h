// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Box.h"
#include "Math/Range.h"
#include "GeometryCacheTrack.generated.h"

#define UE_API GEOMETRYCACHE_API

struct FResourceSizeEx;

struct FGeometryCacheMeshData;
struct FGeometryCacheTrackSampleInfo;

/** Base class for GeometryCache tracks, stores matrix animation data and implements functionality for it */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, BlueprintType, config = Engine)
class UGeometryCacheTrack : public UObject
{
	GENERATED_UCLASS_BODY()

	UE_API virtual ~UGeometryCacheTrack();
	
	//~ Begin UObject Interface.
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface
			
	/**
	 * UpdateMatrixData
	 *
	 * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation should be played on a loop
	 * @param InOutMatrixSampleIndex - Hold the MatrixSampleIndex and will be updated if changed according to the Elapsed Time
	 * @param OutWorldMatrix - Will hold the new WorldMatrix if the SampleIndex changed
	 * @return const bool
	 */
	UE_API virtual const bool UpdateMatrixData(const float Time, const bool bLooping, int32& InOutMatrixSampleIndex, FMatrix& OutWorldMatrix);

	/**
	 * UpdateBoundsData
	 *
	 * Note: Bounds may be sampled at a different rate than the matrixes above so they have separate sample indexes to cache.
	 *
	 * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation should be played on a loop
	 * @param InOutBoundsSampleIndex - Hold the BoundsSampleIndex and will be updated if changed according to the Elapsed Time
	 * @param OutBounds - Will hold the new bounding box if the SampleIndex changed
	 * @return const bool
	 */
	UE_API virtual const bool UpdateBoundsData(const float Time, const bool bLooping, const bool bIsPlayingBackward, int32& InOutBoundsSampleIndex, FBox& OutBounds);

	/**
	 * UpdateMeshData
	 *
	 * * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation should be played on a loop
	 * @param InOutMeshSampleIndex - Hold the MeshSampleIndex and will be updated if changed according to the Elapsed Time
	 * @param OutVertices - Will hold the new VertexData if the SampleIndex changed
	 * @return const bool
	 */
	UE_API virtual const bool UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData );

	/**
	 * SetMatrixSamples, Set the Matrix animation Samples 
	 *
	 * @param Matrices - Array of Matrices
	 * @param SampleTimes - Array of SampleTimes
	 * @return void
	 */
	UE_API virtual void SetMatrixSamples(const TArray<FMatrix>& Matrices, const TArray<float>& SampleTimes);

	/**
	 * AddMatrixSample, Adds a single matrix animation sample (recalculates duration according to SampleTime)
	 *
	 * @param Matrix
	 * @param SampleTime - Time for this sample
	 * @return void
	 */
	UE_API virtual void AddMatrixSample(const FMatrix& Matrix, const float SampleTime);

	/**
	 * Set the duration property.
	 *
	 * @param NewDuration - Duration to set
	 * @return void
	 */
	UE_API virtual void SetDuration(float NewDuration);

	/**
	 * Get the value of the duration property.
	 *
	 * @return The value of the Duration property
	 */
	UE_API virtual float GetDuration();

	/**
	 * GetMaxSampleTime, returns the time for the last sample
	 * Not the same as the animation length since it might not start at time 0
	 *
	 * @return const float
	 */
	UE_API virtual const float GetMaxSampleTime() const;
	
	/**
	 * GetNumMaterials, total number of materials inside this track (depends on batches)
	 *
	 * @return const uint32
	 */
	const uint32 GetNumMaterials() const { return NumMaterials; }

	/**
	 * Get the info for the sample displayed at the given time.
	 * 
	 * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation is being played in a loop
	 */
	UE_API virtual const FGeometryCacheTrackSampleInfo& GetSampleInfo(float Time, const bool bLooping);

	/**
	 * Get the mesh data for the specified time
	 * 
	 * @param Time - The time (in secs) to retrieve the mesh data for
	 * @param OutMeshData - Will contain the mesh data that was retrieved, if successful
	 * @return True if the mesh data could be retrieved
	 */
	virtual bool GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData) { return false; }

	/**
	 * Get the mesh data for the specified sample index
	 *
	 * @param SampleIndex - The sample index to get the mesh data for
	 * @param OutMeshData - Will contain the mesh data that was retrieved, if successful
	 * @return True if the mesh data could be retrieved
	 */
	virtual bool GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData) { return false; }

	/** Return the hash of the mesh data of the track */
	virtual uint64 GetHash() const { return 0; }

	/** Update the current time of the track */
	virtual void UpdateTime(float Time, bool bLooping) { }

protected:

	/** The duration of this track's animation. This is an open ended interval [0..Duration[.
	 * If the animation is looping this is also the length of the loop.
	 *
	 * Note: This is set by the importer possibly based on user preferences. There may be less actual frames available.
	 * E.g. the animation has data for the first 2 seconds, but duration is set to 5, so it will loop every 5 seconds
	 * with the last three seconds showing a static scene.
	 */
	UPROPERTY(VisibleAnywhere, Category = GeometryCache)
	float Duration;

	/**
	 * FindSampleIndexFromTime uses binary search to find the closest index to Time inside Samples
	 *
	 * @param SampleTimes - Array of Sample times used for the search
	 * @param Time - Time for which the closest index has to be found
	 * @param bLooping - Whether or not we should fmod Time according to the last entry in SampleTimes
	 * @return const uint32
	 */
	UE_API const uint32 FindSampleIndexFromTime(const TArray<float>& SampleTimes, const float Time, const bool bLooping);

	/** Matrix sample data, both FMatrix and time*/
	TArray<FMatrix> MatrixSamples;	
	TArray<float> MatrixSampleTimes;

	/** Number of materials for this track*/
	uint32 NumMaterials;
};

/**
 Info stored per sample that is always resident in memory
 */
struct FGeometryCacheTrackSampleInfo
{
	float SampleTime;
	FBox BoundingBox;
	int32 NumVertices;
	int32 NumIndices;

	FGeometryCacheTrackSampleInfo() : SampleTime(0.0f), BoundingBox(EForceInit::ForceInit), NumVertices(0), NumIndices(0) {}

	FGeometryCacheTrackSampleInfo(float SetSampleTime, FBox SetBoundingBox, int32 SetNumVertices, int32 SetNumIndices) :
		SampleTime(SetSampleTime), BoundingBox(SetBoundingBox), NumVertices(SetNumVertices), NumIndices(SetNumIndices) {}

	static UE_API const FGeometryCacheTrackSampleInfo EmptySampleInfo;
};

/*
Hold the visibility state for a given time range
*/
struct FVisibilitySample
{
	FVisibilitySample() = default;

	FVisibilitySample(bool bVisible)
	: bVisibilityState(bVisible)
	{}

	TRange<float> Range;
	bool bVisibilityState;

	friend FArchive& operator<<(FArchive& Ar, FVisibilitySample& Range);

	static UE_API const FVisibilitySample VisibleSample;
	static UE_API const FVisibilitySample InvisibleSample;
};

#undef UE_API
