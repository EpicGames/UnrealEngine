// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCacheTrack.h"

#include "GeometryCacheTrackFlipbookAnimation.generated.h"

#define UE_API GEOMETRYCACHE_API

/** Derived GeometryCacheTrack class, used for Transform animation. */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, BlueprintType, config = Engine, deprecated)
class UDEPRECATED_GeometryCacheTrack_FlipbookAnimation : public UGeometryCacheTrack
{
	GENERATED_UCLASS_BODY()

	UE_API virtual ~UDEPRECATED_GeometryCacheTrack_FlipbookAnimation();

	//~ Begin UObject Interface.
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

	//~ Begin UGeometryCacheTrack Interface.
	UE_API virtual const bool UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData) override;
	UE_API virtual const float GetMaxSampleTime() const override;
	//~ End UGeometryCacheTrack Interface.

	/**
	* Add a GeometryCacheMeshData sample to the Track
	*
	* @param MeshData - Holds the mesh data for the specific sample
	* @param SampleTime - SampleTime for the specific sample being added
	* @return void
	*/
	UFUNCTION()
	UE_API void AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime);

private:
	/** Number of Mesh Sample within this track */
	UPROPERTY(VisibleAnywhere, Category = GeometryCache)
	uint32 NumMeshSamples;

	/** Stored data for each Mesh sample */
	TArray<FGeometryCacheMeshData> MeshSamples;
	TArray<float> MeshSampleTimes;
};

#undef UE_API
