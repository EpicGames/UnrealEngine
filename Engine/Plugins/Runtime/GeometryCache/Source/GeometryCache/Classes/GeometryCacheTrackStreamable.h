// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCacheCodecBase.h"
#include "GeometryCacheTrack.h"
#include "RenderResource.h"
#include "RenderCommandFence.h"

#include "GeometryCacheTrackStreamable.generated.h"

#define UE_API GEOMETRYCACHE_API

class UGeometryCacheTrackStreamable;

class FGeometryCachePreprocessor;

struct FGeometryCacheTrackMeshDataUpdate
{
	void* VertexBuffer;
	void* IndexBuffer;
};

/**
	All render thread state for a geometry cache track. This contains shared render thread
	state shared by all GeometryCacheComponents that use the same GeometryCache. The
	per-component state is managed in the GeometryCacheScene proxy.
*/
class FGeometryCacheTrackStreamableRenderResource : public FRenderResource
{
public :
	FGeometryCacheTrackStreamableRenderResource();
	void InitGame(UGeometryCacheTrackStreamable *Track);
	void InitRHI(FRHICommandListBase& RHICmdList) override;
	void ReleaseRHI() override;

	bool UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData);

	/**
	 * Decode a single frame of mesh data.
	 * @param SampleIndexToDecode - Index of the frame to decode. Some codecs may have inter-frame dependencies. All complexity related to 
	 *								is handled by the codec but decoding subsequent frames may be more efficient than decoding random frames.
	 * @param OutMeshData - The decoded mesh is stored in this object.
	 */
	bool DecodeMeshData(int32 SampleIndexToDecode, FGeometryCacheMeshData& OutMeshData);

	bool IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB);

	/**
		Get the UGeometryCacheTrackStreamable track corresponding to this render resource.

		This track data is valid and won't change as long as this FGeometryCacheTrackStreamableRenderResource
		instance is live. If we need to modify the track we will first tear down this FGeometryCacheTrackStreamableRenderResource
		instance and sync the renderthread.

		So you can keep the pointer around as long as you know the FGeometryCacheTrackStreamableRenderResource instance 
		you got it from stays valid.
	*/
	UGeometryCacheTrackStreamable *GetTrack()
	{
		return Track;
	}

	FGeometryCacheCodecRenderStateBase *GetCodec()
	{
		return Codec;
	}

protected:
	FGeometryCacheCodecRenderStateBase *Codec; //Render thread codec instance
	UGeometryCacheTrackStreamable *Track; //See docs for GetTrack above.
};

/**
	Info stored per sample that is always resident in memory and does not require parsing the chunks.
	Needed to keep support for serialization of FGeometryCacheTrackStreamableSampleInfo
*/
struct FGeometryCacheTrackStreamableSampleInfo : public FGeometryCacheTrackSampleInfo
{
	FGeometryCacheTrackStreamableSampleInfo() : FGeometryCacheTrackSampleInfo() {}

	FGeometryCacheTrackStreamableSampleInfo(float SetSampleTime, FBox SetBoundingBox, int32 SetNumVertices, int32 SetNumIndices) :
		FGeometryCacheTrackSampleInfo(SetSampleTime, SetBoundingBox, SetNumVertices, SetNumIndices) {}

	friend FArchive& operator<<(FArchive& Ar, FGeometryCacheTrackStreamableSampleInfo& Info);
};

/** 
	Derived GeometryCacheTrack class, used for Transform animation.

	\note FGeometryCacheTrackStreamableRenderResource keeps a reference to the track.
	Be sure to keep the implementation of this class valid so it properly releases
	the render resoruce before making any changes to this object that may affect the render thread.
*/
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, BlueprintType, config = Engine)
class UGeometryCacheTrackStreamable : public UGeometryCacheTrack
{
	GENERATED_UCLASS_BODY()

	UE_API virtual ~UGeometryCacheTrackStreamable();

	//~ Begin UObject Interface.
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void FinishDestroy() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface.

	//~ Begin UGeometryCacheTrack Interface.
	UE_API virtual const bool UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData) override;
	UE_API virtual const bool UpdateBoundsData(const float Time, const bool bLooping, const bool bIsPlayingBackward, int32& InOutBoundsSampleIndex, FBox& OutBounds) override;
	UE_API virtual const float GetMaxSampleTime() const override;
	UE_API virtual void SetDuration(float NewDuration) override;
	UE_API virtual const FGeometryCacheTrackSampleInfo& GetSampleInfo(float Time, const bool bLooping) override;
	UE_API virtual bool GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData) override;
	UE_API virtual bool GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	UE_API virtual uint64 GetHash() const override;
	//~ End UGeometryCacheTrack Interface.

#if WITH_EDITORONLY_DATA
	/**
		Begin coding and set the codec to use for this track.
		The passed in codec object is assumed to be exclusive to this track.
	*/
	UE_API void BeginCoding(UGeometryCacheCodecBase* SetCodec, bool bForceSingleOptimization, bool bCalculateAndStoreMotionVectors, bool bOptimizeIndexBuffers);
	
	/**
	* Add a GeometryCacheMeshData sample to the Track
	*
	* @param MeshData - Holds the mesh data for the specific sample
	* @param SampleTime - SampleTime for the specific sample being added
	* @return void
	*/
	UE_API void AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious);

	UE_API void AddVisibilitySample(const bool bVisible, const float SampleTime);

	/**
		Finish up coding. Return true if the track has samples
	*/
	UE_API bool EndCoding();
#endif

	/** Codec for this track */
	UPROPERTY(VisibleAnywhere, Category = GeometryCache)
	TObjectPtr<UGeometryCacheCodecBase> Codec;
	FGeometryCachePreprocessor *Preprocessor;

	/**
		Get the CunksIds that need to be loaded to display any frames falling within the given time range.
		@param StartTime Beginning of the range to return chunks for
		@param EndTime End of the range to return chunks for
		@param Looping If the animation playback is looping and thus the interval needs to wrap around based on this track's Duration
		@param OutChunkIndexes This list will be filled with the needed ChunkIds
	*/
	UE_API void GetChunksForTimeRange(float StartTime, float EndTime, bool Looping, TArray<int32>& OutChunkIndexes);

	const FStreamedGeometryCacheChunk &GetChunk(int32 ChunkID) const
	{
		return Chunks[ChunkID];
	}

	FStreamedGeometryCacheChunk &GetChunk(int32 ChunkID)
	{
		return Chunks[ChunkID];
	}

	FGeometryCacheTrackStreamableRenderResource *GetRenderResource()
	{
		// This is valid as long as the uobject is valid...
		return &RenderResource;
	}

	friend class FGeometryCacheTrackStreamableRenderResource;

	/**
	* FindSampleIndexFromTime uses binary search to find the closest index to Time inside Samples
	*
	* @param Time - Time for which the closest index has to be found
	* @param bLooping - Whether or not we should fmod Time according to the last entry in SampleTimes
	* @return const uint32
	*/
	UE_API const uint32 FindSampleIndexFromTime(const float Time, const bool bLooping) const;

	/**
	 * Find the two frames closest to the given time.
	 * InterpolationFactor gives the position of the requested time slot between the two returned frames.
	 * 0.0 => We are very close to OutFrameIndex
	 * 1.0 => We are very close to OutNextFrameIndex
	 * If bIsPlayingBackwards it will return exactly the same indexes but in the reversed order. The
	 * InterpolationFactor will also be updated accordingly
	 */
	UE_API void FindSampleIndexesFromTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32 &OutFrameIndex, int32 &OutNextFrameIndex, float &InterpolationFactor);

	/**
		Get the info for the sample with the given ID.
	*/
	UE_API const FGeometryCacheTrackStreamableSampleInfo& GetSampleInfo(int32 SampleID) const;

	UE_API const FVisibilitySample& GetVisibilitySample(float Time, const bool bLooping) const;

	friend class FCodecGeometryCachePreprocessor;

	static UE_API void TriggerSerializationCrash();

private:

	UE_API void ReleaseRenderResources();
	UE_API void InitializeRenderResources();

	/** Stored data for each Mesh sample */
	TArray<FStreamedGeometryCacheChunk> Chunks;
	TArray<FGeometryCacheTrackStreamableSampleInfo> Samples;

	TArray<FVisibilitySample> VisibilitySamples;
#if WITH_EDITOR
	TArray<TPair<float, bool>> ImportVisibilitySamples;
#endif // WITH_EDITOR

	FGeometryCacheTrackStreamableRenderResource RenderResource;
	FRenderCommandFence ReleaseResourcesFence;

	UPROPERTY()
	float StartSampleTime;

	uint64 Hash;
};

#undef UE_API
