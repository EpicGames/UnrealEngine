// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCacheTrack.h"
#include "GeometryCacheMeshData.h"

#include "GeometryCacheTrackTransformGroupAnimation.generated.h"

#define UE_API GEOMETRYCACHE_API

/** Derived GeometryCacheTrack class, used for Transform animation. */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, BlueprintType, config = Engine, deprecated)
class UDEPRECATED_GeometryCacheTrack_TransformGroupAnimation : public UGeometryCacheTrack
{
	GENERATED_UCLASS_BODY()
		
	// Begin UObject interface.
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	// End UObject interface.
	
	// Begin UGeometryCacheTrack interface.
	UE_API virtual const bool UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData) override;
	// End UGeometryCacheTrack interface.

	/**
	* Sets/updates the MeshData for this track
	*
	* @param NewMeshData - GeometryCacheMeshData instance later used as the rendered mesh	
	*/
	UFUNCTION()
	UE_API void SetMesh(const FGeometryCacheMeshData& NewMeshData);	
private:
	FGeometryCacheMeshData MeshData;
};

#undef UE_API
