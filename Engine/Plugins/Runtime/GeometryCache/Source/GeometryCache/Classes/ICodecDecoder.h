// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FBufferReader;
struct FGeometryCacheMeshData;

/**
* Interface to the internal mesh compressor's decoder in order to hide implementation
*/
class ICodecDecoder
{
public:
	virtual ~ICodecDecoder() = default;

	/**
	* Read a frame's bit stream and decode the frame
	*
	* @param Reader - Reader holding the frame's bit stream
	* @param OutMeshData - Decoded mesh
	*/
	virtual bool DecodeFrameData(FBufferReader& Reader, FGeometryCacheMeshData& OutMeshData) = 0;
};
