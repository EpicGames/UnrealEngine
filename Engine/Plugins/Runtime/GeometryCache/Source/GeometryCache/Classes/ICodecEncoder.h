// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FMemoryWriter;
struct FGeometryCacheCodecEncodeArguments;

/**
* Interface to the internal mesh compressor's encoder in order to hide implementation
*/
class ICodecEncoder
{
public:
	virtual ~ICodecEncoder() = default;

	/**
	* Encode a frame and write the bitstream
	*
	* @param Writer - Writer for the output bit stream
	* @param Args - Encoding arguments
	*/
	virtual bool EncodeFrameData(FMemoryWriter& Writer, const FGeometryCacheCodecEncodeArguments &Args) = 0;
};
