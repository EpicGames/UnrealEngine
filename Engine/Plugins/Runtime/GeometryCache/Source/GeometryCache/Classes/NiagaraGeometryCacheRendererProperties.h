// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataSetAccessor.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraGeometryCacheRendererProperties.generated.h"

class UMaterialInstanceConstant;
class UGeometryCache;
struct FNiagaraVariable;
struct FVersionedNiagaraEmitter;

class FNiagaraEmitterInstance;
class SWidget;

USTRUCT()
struct FNiagaraGeometryCacheMICOverride
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UMaterialInterface> OriginalMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceConstant> ReplacementMaterial;
};

USTRUCT()
struct FNiagaraGeometryCacheReference
{
	GENERATED_USTRUCT_BODY()

	FNiagaraGeometryCacheReference();

	UMaterialInterface* ResolveMaterial(UGeometryCache* ResolvedCache, int32 MaterialIndex) const;

	/** Reference to the geometry cache asset to use (if the user parameter binding is not set) */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	TObjectPtr<UGeometryCache> GeometryCache;

	/** Use the UGeometryCache bound to this user variable if it is set to a valid value. If this is bound to a valid value and GeometryCache is also set, GeometryCacheUserParamBinding wins.*/
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraUserParameterBinding GeometryCacheUserParamBinding;

	/** The materials to be used instead of the GeometryCache's materials. If the GeometryCache requires more materials than exist in this array or any entry in this array is set to None, we will use the GeometryCache's existing material instead.*/
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY()
	TArray<FNiagaraGeometryCacheMICOverride> MICOverrideMaterials;
};

UCLASS(editinlinenew, MinimalAPI, meta = (DisplayName = "Geometry Cache Renderer"))
class UNiagaraGeometryCacheRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	UNiagaraGeometryCacheRendererProperties();

	//UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif// WITH_EDITORONLY_DATA
	//UObject Interface END

	static void InitCDOPropertiesAfterModuleStartup();

	//~ UNiagaraRendererProperties interface
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override { return nullptr; }
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return InSimTarget == ENiagaraSimTarget::CPUSim; };
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	virtual void GetUsedMeshes(const FNiagaraEmitterInstance* InEmitter, TArray<UObject*>& OutMeshes) const override;
	virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;
	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const override { return SourceMode; }
	virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)  override;
	virtual bool NeedsMIDsForMaterials() const override	{ return MaterialParameters.HasAnyBindings(); }
	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;

#if WITH_EDITORONLY_DATA
	virtual void GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual const FSlateBrush* GetStackIcon() const override;
	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	virtual TArray<FNiagaraVariable> GetBoundAttributes() const override;
	virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
#endif // WITH_EDITORONLY_DATA

	void UpdateMICs();

	static UGeometryCache* ResolveGeometryCache(const FNiagaraGeometryCacheReference& Entry, const FNiagaraEmitterInstance* Emitter);

	/** Reference to the geometry cache assets to use. If ArrayIndexBinding is not set, a random element is used for each particle. */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	TArray<FNiagaraGeometryCacheReference> GeometryCaches;

	/** Whether or not to draw a single element for the Emitter or to draw the particles.*/
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	ENiagaraRendererSourceDataMode SourceMode = ENiagaraRendererSourceDataMode::Particles;

	/** If true, then the geometry cache keeps playing in a loop */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	bool bIsLooping = true;

	/** The max number of components that this emitter will spawn or update each frame. */
	UPROPERTY(EditAnywhere, Category = "GeometryCache", meta = (ClampMin = 1))
	uint32 ComponentCountLimit = 15;
	
	/** Which attribute should we use for the geometry cache position? */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Which attribute should we use for the geometry cache rotation? */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding RotationBinding;

	/** Which attribute should we use for the geometry cache component scale? */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding ScaleBinding;

	/** Which attribute should we use for the geometry cache's current animation time? */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding ElapsedTimeBinding;

	/** Which attribute should we use to check if rendering should be enabled? */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding EnabledBinding;

	/** Which attribute should we use to pick the element in the geometry cache array for this renderer? If not set, a random element will be used. */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding ArrayIndexBinding;

	/** Which attribute should we use for the renderer visibility tag? */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "GeometryCache")
	int32 RendererVisibility;

	/** If true then components will not be automatically assigned to the first particle available, but try to stick to the same particle based on its unique id.
	* Disabling this option is faster, but a particle can get a different component each tick, which can lead to problems with for example motion blur. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "GeometryCache")
	bool bAssignComponentsOnParticleID = true;
	
	/** Determines if the geometry component will affect the engine's physics system with its collision shape. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Component")
	TEnumAsByte<ECollisionEnabled::Type> EnableCollision = ECollisionEnabled::NoCollision;
	
	/** If true then the geometry component will invalidate the nav mesh on movement. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Component")
	bool bAffectsNavigation = false;

	/** If this array has entries, we will create a MaterialInstanceDynamic per Emitter instance from Material and set the Material parameters using the Niagara simulation variables listed.*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraRendererMaterialParameters MaterialParameters;

	FNiagaraDataSetAccessor<FNiagaraPosition>	PositionAccessor;
	FNiagaraDataSetAccessor<FVector3f>			RotationAccessor;
	FNiagaraDataSetAccessor<FVector3f>			ScaleAccessor;
	FNiagaraDataSetAccessor<float>				ElapsedTimeAccessor;
	FNiagaraDataSetAccessor<FNiagaraBool>		EnabledAccessor;
	FNiagaraDataSetAccessor<int32>				ArrayIndexAccessor;
	FNiagaraDataSetAccessor<int32>				VisTagAccessor;
	FNiagaraDataSetAccessor<int32>				UniqueIDAccessor;

	virtual bool NeedsSystemPostTick() const override { return true; }
	virtual bool NeedsSystemCompletion() const override { return true; }

protected:
	void InitBindings();

	static FNiagaraVariable Particles_GeoCacheRotation;
	static FNiagaraVariable Particles_Age;
	static FNiagaraVariable Particles_GeoCacheIsEnabled;
	static FNiagaraVariable Particles_ArrayIndex;

	static void InitDefaultAttributes();
private:
	static TArray<TWeakObjectPtr<UNiagaraGeometryCacheRendererProperties>> GeometryCacheRendererPropertiesToDeferredInit;
};
