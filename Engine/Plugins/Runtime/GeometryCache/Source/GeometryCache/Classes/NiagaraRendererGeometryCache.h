// Copyright Epic Games, Inc. All Rights Reserved.

/*========================================================================================
NiagaraRendererGeometryCache.h: Renderer for geometry cache components.
=========================================================================================*/
#pragma once

#include "NiagaraRenderer.h"

class UGeometryCacheComponent;

class UNiagaraGeometryCacheRendererProperties;
class FNiagaraDataSet;

/**
* NiagaraRendererGeometryCache renders a geometry cache asset
*/
class FNiagaraRendererGeometryCache : public FNiagaraRenderer
{
public:

	FNiagaraRendererGeometryCache(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	virtual ~FNiagaraRendererGeometryCache();

	//FNiagaraRenderer interface
	virtual void Initialize(const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	virtual void DestroyRenderState_Concurrent() override;
	virtual void PostSystemTick_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) override;
	virtual void OnSystemComplete_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) override;
	//FNiagaraRenderer interface END

private:
	struct FComponentPoolEntry
	{
		TWeakObjectPtr<UGeometryCacheComponent> Component;
		float LastActiveTime = 0.0f;
		float LastElapsedTime = 0.0f;
		int32 LastAssignedToParticleID = -1;
	};

	UGeometryCacheComponent* CreateOrGetPooledComponent(
		const UNiagaraGeometryCacheRendererProperties* Properties,
		USceneComponent* AttachComponent,
		const FNiagaraEmitterInstance* Emitter,
		TOptional<int32> DefaultCacheIndex,
		uint32 ParticleIndex,
		int32 ParticleID,
		int32& InOutPoolIndex
	);

	void ResetComponentPool(bool bResetOwner);

	// if the niagara component is not attached to an actor, we need to spawn and keep track of a temporary actor
	TWeakObjectPtr<AActor> SpawnedOwner;

	// all of the spawned components
	TArray<FComponentPoolEntry> ComponentPool;

	// Used to remap materials into the compact list so we can use MIDs
	TArray<TArray<int32>> MaterialRemapTable;
};
