// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryCache : ModuleRules
{
	public GeometryCache(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
                "InputCore",
                "RenderCore",
                "RHI",
                "Niagara",
                "NiagaraCore"
			}
		);
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"GeometryCore",
				"MeshConversion",
				"MeshDescription",
				"StaticMeshDescription",
				"TraceLog",
			}
		);

		PublicIncludePathModuleNames.Add("TargetPlatform");
		PrivateIncludePathModuleNames.Add("Shaders");

		if (Target.bBuildEditor)
        {
            PublicIncludePathModuleNames.Add("GeometryCacheEd");
            DynamicallyLoadedModuleNames.Add("GeometryCacheEd");
            PrivateDependencyModuleNames.Add("MeshUtilitiesCommon");
            PrivateDependencyModuleNames.Add("UnrealEd");
        }

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
