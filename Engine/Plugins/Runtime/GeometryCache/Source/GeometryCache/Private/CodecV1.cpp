// Copyright Epic Games, Inc. All Rights Reserved.
#include "CodecV1.h"
#include "GeometryCacheCodecBase.h"
#include "GeometryCacheMeshData.h"
#include "RingBuffer.h"
#include "Serialization/MemoryWriter.h"
#include "GeometryCacheModule.h"
#include "Async/ParallelFor.h"

#define V1_MAGIC 123
#define V2_MAGIC 124
#define V3_MAGIC 125 // Introduction of ImportedVertexNumbers

#define MAX_CHUNK_VERTICES	1024u
#define CHUNK_MAX_TRIANGLES	2048u
#define MAX_CHUNK_INDICES	(CHUNK_MAX_TRIANGLES * 3)
/*
//#define DEBUG_DUMP_FRAMES // Dump raw frames to file during encoding for debugging
//#define DEBUG_CODECTEST_DUMPED_FRAMES // Test codec using dumped raw files

#ifdef DEBUG_CODECTEST_DUMPED_FRAMES
#ifndef DEBUG_DUMP_FRAMES
// Standalone codec for the test, reads raw dumps of frames, encodes and decodes them.
static CodecV1Test CodecV1Test(TEXT("F:\\"));		// Test run in constructor
#endif
#endif
*/

static TAutoConsoleVariable<int32> CVarCodecDebug(
	TEXT("GeometryCache.Codec.Debug"),
	0,
	TEXT("Enables debug logging for the codec."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

DEFINE_LOG_CATEGORY(LogGeoCaStreamingCodecV1);

// At start of frame
struct FCodedFrameHeader
{
	uint32 Magic;
	uint32 PayloadSize;
	uint32 IndexCount;
	uint32 VertexCount;
};

// At start of vertex position stream
struct FVertexStreamHeader
{	
	float QuantizationPrecision;
	FIntVector Translation;
};

// At start of UV stream
struct FUVStreamHeader
{
	uint32 QuantizationBits;
	FVector2f Range;
};

// At start of UV stream
struct FMotionVectorStreamHeader
{
	float QuantizationPrecision;
};

/** Timer returning milliseconds, for fast iteration development */
class FExperimentTimer
{
public:
	FExperimentTimer()
	{
		StartTime = FPlatformTime::Seconds();
	}

	double Get()
	{
		return (FPlatformTime::Seconds() - StartTime) * 1000.0;
	}

private:
	double StartTime;
};

/** Quantizer, discretizes a continuous range of values into bins */
class FQuantizer
{
public:	
	/** Initialize with a fixed precision (which is the bin size) */
	FQuantizer(float Precision)
	{
		BinSize = Precision;
		HalfBinSize = BinSize / 2.0f;
		OneOverBinSize = 1.0f / BinSize;
	}

	/** Initialize with a range and a number of bits. NumBits determines the number of bins we have, 
	    and Range determines the sizes of the bins. */
	FQuantizer(float Range, int32 NumBits)
	{		
		int32 BinCount = (int)FMath::Pow(2.0f, NumBits);
		BinSize = Range / static_cast<float>(BinCount);
		HalfBinSize = Range / static_cast<float>(BinCount) / 2.0f;
		OneOverBinSize = static_cast<float>(BinCount) / Range;
	}
#if WITH_EDITOR
	/** Quantize a value */
	FORCEINLINE int32 QuantizeFloat(float Value)
	{
		// We compensate for energy loss around zero, e.g., Given a bin size 1, we map [-0.5,0.5[ -> 0, [-1.5,-0.5[ -> -1, [0.5,1.5[ -> 1, 
		int32 Negative = (int)(Value >= 0.0f) * 2 - 1; // Positive: 1, negative: -1
		int32 IntValue = static_cast<int32>((FMath::Abs(Value) + HalfBinSize) * OneOverBinSize);
		return IntValue * Negative;
	}
#endif // WITH_EDITOR

	/** Dequantize a quantized value */
	FORCEINLINE float DequantizeFloat(int32 Value)
	{
		return (float)Value * BinSize;
	}

private:
	float BinSize;
	float HalfBinSize;
	float OneOverBinSize;
};

/** Quantizer for FVector2Ds, discretizes a continuous 2D range of values into bins */
class FQuantizerVector2
{
public:
	/** Initialize with a fixed precision (which is the bin size) */
	FQuantizerVector2(float Precision) : QuantizerX(Precision), QuantizerY(Precision)
	{
	}

	/** Initialize with a range and a number of bits. NumBits determines the number of bins we have,
		and Range determines the sizes of the bins. */
	FQuantizerVector2(const FVector2f& Range, int32 Bits) : QuantizerX(Range.GetMax(), Bits), QuantizerY(Range.GetMax(), Bits)
	{
	}

#if WITH_EDITOR
	/** Quantize a value */
	FORCEINLINE FIntVector Quantize(const FVector2f& Value)
	{
		return FIntVector(QuantizerX.QuantizeFloat(Value.X), QuantizerY.QuantizeFloat(Value.Y), 0);
	}
#endif // WITH_EDITOR

	/** Dequantize a quantized value */
	FORCEINLINE FVector2f Dequantize(const FIntVector& Value)
	{
		return FVector2f(QuantizerX.DequantizeFloat(Value.X), QuantizerY.DequantizeFloat(Value.Y));
	}

private:
	FQuantizer QuantizerX;
	FQuantizer QuantizerY;
};

/** Quantizer for FVectors, discretizes a continuous 3D range of values into bins */
class FQuantizerVector3
{
public:
	/** Initialize with a fixed precision (which is the bin size) */
	FQuantizerVector3(float Precision) : Quantizer(Precision)
	{
	}

	/** Initialize with a range and a number of bits. NumBits determines the number of bins we have,
		and Range determines the sizes of the bins. */
	FQuantizerVector3(const FVector3f& Range, int32 Bits) : Quantizer(Range.GetMax(), Bits)
	{
	}

#if WITH_EDITOR
	/** Quantize a value */
	FORCEINLINE FIntVector Quantize(const FVector3f& Value)
	{
		return FIntVector(Quantizer.QuantizeFloat(Value.X), Quantizer.QuantizeFloat(Value.Y), Quantizer.QuantizeFloat(Value.Z));
	}
#endif // WITH_EDITOR

	/** Dequantize a quantized value */
	FORCEINLINE FVector3f Dequantize(const FIntVector& Value)
	{
		return FVector3f(Quantizer.DequantizeFloat(Value.X), Quantizer.DequantizeFloat(Value.Y), Quantizer.DequantizeFloat(Value.Z));
	}

private:
	FQuantizer Quantizer;	
};

// Custom serialization for our const TArray because of the const
FArchive& operator<<(FArchive &Ar, const TArray<FGeometryCacheMeshBatchInfo>& BatchesInfo)
{
	check(Ar.IsSaving());
	int32 Num = BatchesInfo.Num();
	Ar << Num;

	for (int32 Index = 0; Index < Num; ++Index)
	{
		FGeometryCacheMeshBatchInfo NonConstCopy = BatchesInfo[Index];
		Ar << NonConstCopy;
	}

	return Ar;
}

// Custom serialization for our const FBox because of the const
FArchive& operator<<(FArchive &Ar, const FBox3f& Box)
{
	check(Ar.IsSaving());
	FBox3f NonConstBox = Box; // copy
	Ar << NonConstBox;
	return Ar;
}

#if WITH_EDITOR
FCodecV1Encoder::FCodecV1Encoder() : Config(FCodecV1EncoderConfig::DefaultConfig())
{
	EncodingContext = { 0 };

	SetupTables();	// Create our huffman tables
}

FCodecV1Encoder::FCodecV1Encoder(const FCodecV1EncoderConfig& EncoderConfig) : Config(EncoderConfig)
{
	EncodingContext = { 0 };

	SetupTables();	// Create our huffman tables
}

void FCodecV1Encoder::EncodeIndexStream(const uint32* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats)
{	
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer); // Count the number of bytes we are writing

	FRingBuffer<uint32, IndexStreamCodingHistorySize> LastReconstructed(IndexStreamCodingHistorySize); // History holding previously seen indices

	uint8* RawElementData = (uint8*)Stream;

	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffsetBytes)
	{
		// Load data
		uint32 Value = *(uint32*)RawElementData;

		uint32 Prediction = LastReconstructed[0]; // Delta coding, best effort
		int32 Residual = Value - Prediction;

		// Write residual	
		WriteInt32(EncodingContext.ResidualIndicesTable, Residual);		

		// Store previous encountered values
		uint32 Reconstructed = Prediction + Residual;
		LastReconstructed.Push(Reconstructed);
	}

	// Gather rate and quality statistics
	float Quality = 0.0f;
	Stats = FStreamEncodingStatistics(ByteCounter.Read(), ElementCount * sizeof(uint32), Quality);
}

void FCodecV1Encoder::EncodePositionStream(const FVector3f* VertexStream, uint64 VertexElementOffset, uint32 VertexElementCount, FStreamEncodingStatistics& Stats)
{
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer);

	// Quantizer	
	const float QuantizationPrecision = Config.VertexQuantizationPrecision;
	FQuantizerVector3 Quantizer(QuantizationPrecision);
	
	// Bounding box and translation
	const FBox3f& BoundingBox = EncodingContext.MeshData->BoundingBox;
	FIntVector QuantizedBoxMin = Quantizer.Quantize(BoundingBox.Min); // Quantize the bounds of the bounding box
	FIntVector QuantizedBoxMax = Quantizer.Quantize(BoundingBox.Max);
	FIntVector QuantizedBoxCenter = (QuantizedBoxMax + QuantizedBoxMin) / 2; // Calculate the center of our new quantized bounding box
	FIntVector QuantizedTranslationToCenter = QuantizedBoxCenter; // Translation vector to move the mesh to the center of the quantized bounding box

	// Write header
	FVertexStreamHeader Header;	
	Header.QuantizationPrecision = QuantizationPrecision;
	Header.Translation = QuantizedTranslationToCenter;
	WriteBytes((void*)&Header, sizeof(Header));
	
	FIntVector Prediction(0, 0, 0);	// Previously seen position
		
	FQualityMetric QualityMetric;

	const uint8* RawElementDataVertices = (const uint8*)VertexStream;

	// Walk over indices/triangles
	for (uint32 ElementIdx = 0; ElementIdx < VertexElementCount; ++ElementIdx)
	{
		// Code a newly encountered vertex
		FVector3f VertexValue = *(FVector3f*)RawElementDataVertices;
		RawElementDataVertices += VertexElementOffset;

		// Quantize
		FIntVector Encoded = Quantizer.Quantize(VertexValue);

		// Translate to center
		FIntVector EncodedCentered = Encoded - QuantizedTranslationToCenter;

		// Residual to code
		FIntVector Residual = EncodedCentered - Prediction;
				
		// Write residual
		WriteInt32(EncodingContext.ResidualVertexPosTable, Residual.X);
		WriteInt32(EncodingContext.ResidualVertexPosTable, Residual.Y);
		WriteInt32(EncodingContext.ResidualVertexPosTable, Residual.Z);

		// Store previous encountered values
		FIntVector Reconstructed = Prediction + Residual;

		// Calculate error
		FVector3f DequantReconstructed = Quantizer.Dequantize(Reconstructed);
		QualityMetric.Register(VertexValue, DequantReconstructed);
		Prediction = Reconstructed;
	}

	// Gather rate and quality statistics
	Stats = FStreamEncodingStatistics(ByteCounter.Read(), VertexElementCount * sizeof(FVector3f), QualityMetric.ReadMSE());
}

void FCodecV1Encoder::EncodeColorStream(const FColor* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats)
{	
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer);

	FRingBuffer<FIntVector4, ColorStreamCodingHistorySize> ReconstructedHistory(ColorStreamCodingHistorySize, FIntVector4(128, 128, 128, 255)); // Previously seen colors

	const uint8* RawElementData = (const uint8*)Stream;

	// Walk over colors
	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffsetBytes)
	{
		// Load data
		FColor& ColorValue = *(FColor*)RawElementData;
		FIntVector4 Value(ColorValue.R, ColorValue.G, ColorValue.B, ColorValue.A);

		FIntVector4 Prediction = ReconstructedHistory[0]; 
		FIntVector4 Residual = FCodecV1SharedTools::SubtractVector4(Value, Prediction); // Residual = Value - Prediction

		// We signal a perfect prediction with a skip bit
		bool bEqual = (Residual == FIntVector4(0, 0, 0, 0));
		int32 SkipBit = bEqual ? 1 : 0;
		WriteBits(SkipBit, 1);

		if (!bEqual)
		{
			// No perfect prediction so write the residuals
			WriteInt32(EncodingContext.ResidualColorTable, Residual.X);
			WriteInt32(EncodingContext.ResidualColorTable, Residual.Y);
			WriteInt32(EncodingContext.ResidualColorTable, Residual.Z);
			WriteInt32(EncodingContext.ResidualColorTable, Residual.W);
		}

		// Decode as the decoder would and keep the result for future prediction
		FIntVector4 Reconstructed = FCodecV1SharedTools::SumVector4(Prediction, Residual); // Decode as the decoder will do
		ReconstructedHistory.Push(Reconstructed);
	}

	// Gather rate and quality statistics
	float Quality = 0.0f; // Lossless
	Stats = FStreamEncodingStatistics(ByteCounter.Read(), ElementCount * sizeof(FColor), Quality);
}

void FCodecV1Encoder::EncodeNormalStream(const FPackedNormal* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FHuffmanEncodeTable& Table, FStreamEncodingStatistics& Stats)
{
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer);

	uint8 x = 128, y = 128, z = 128, w = 128;
	
	const uint8* RawElementData = (const uint8*)Stream;

	// Walk over colors
	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffsetBytes)
	{
		// Load data
		FPackedNormal& NormalValue = *(FPackedNormal*)RawElementData;		
		
		int8 dx = static_cast<int8>(NormalValue.Vector.X - x);
		int8 dy = static_cast<int8>(NormalValue.Vector.Y - y);
		int8 dz = static_cast<int8>(NormalValue.Vector.Z - z);
		int8 dw = static_cast<int8>(NormalValue.Vector.W - w);

		// Write residual	
		WriteSymbol(Table, (uint8)dx);
		WriteSymbol(Table, (uint8)dy);
		WriteSymbol(Table, (uint8)dz);
		WriteSymbol(Table, (uint8)dw);

		x = NormalValue.Vector.X;
		y = NormalValue.Vector.Y;
		z = NormalValue.Vector.Z;
		w = NormalValue.Vector.W;
	}

	// Gather rate and quality statistics
	float Quality = 0.0f; // Lossless
	Stats = FStreamEncodingStatistics(ByteCounter.Read(), ElementCount * sizeof(FPackedNormal), Quality);
}

void FCodecV1Encoder::EncodeUVStream(const FVector2f* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats)
{
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer);
	
	// Setup quantizer. We set the range to a static [0-1] even though we can get coordinates out of this range: a static range
	// to avoid jittering of coordinates over frames. Note that out of range values (e.g., [0-6]) will quantize fine, but will take 
	// 'UVQuantizationBitRange' bits for their fraction part
	const int32 BitRange = Config.UVQuantizationBitRange;
	FVector2f Range(1.0f, 1.0f);	
	FQuantizerVector2 Quantizer(Range, BitRange);

	// Write header
	FUVStreamHeader Header;
	Header.QuantizationBits = BitRange;	
	Header.Range = Range;
	WriteBytes((void*)&Header, sizeof(Header));
		
	FRingBuffer<FIntVector, UVStreamCodingHistorySize> ReconstructedHistory(UVStreamCodingHistorySize, FIntVector(0, 0, 0)); // Previously seen UVs
	FQualityMetric2D QualityMetric;

	const uint8* RawElementData = (const uint8*)Stream;

	// Walk over UVs, note, we can get better results if we walk the indices and use knowledge on the triangles to predict the UVs
	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffsetBytes)
	{
		// Load data
		FVector2f& UVValue = *(FVector2f*)RawElementData;

		FIntVector Encoded = Quantizer.Quantize(UVValue);

		FIntVector Prediction = ReconstructedHistory[0]; // Delta coding
		FIntVector Residual = Encoded - Prediction;

		// Write residual	
		WriteInt32(EncodingContext.ResidualUVTable, Residual.X);
		WriteInt32(EncodingContext.ResidualUVTable, Residual.Y);

		// Store previous encountered values
		FIntVector Reconstructed = Prediction + Residual;
		ReconstructedHistory.Push(Reconstructed);

		// Calculate error
		FVector2f DequantReconstructed = Quantizer.Dequantize(Reconstructed);
		QualityMetric.Register(UVValue, DequantReconstructed);
	}

	// Gather rate and quality statistics
	Stats = FStreamEncodingStatistics(ByteCounter.Read(), ElementCount * sizeof(FVector2f), QualityMetric.ReadMSE());
}

void FCodecV1Encoder::EncodeMotionVectorStream(const FVector3f* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats)
{
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer);

	float QuantizationPrecision = Config.VertexQuantizationPrecision; // We use the same precision as the one used for the positions
	FQuantizerVector3 Quantizer(Config.VertexQuantizationPrecision);

	// Write header
	FMotionVectorStreamHeader Header;
	Header.QuantizationPrecision = QuantizationPrecision;
	WriteBytes((void*)&Header, sizeof(Header));

	FRingBuffer<FIntVector, MotionVectorStreamCodingHistorySize> ReconstructedHistory(MotionVectorStreamCodingHistorySize, FIntVector(0, 0, 0)); // Previously seen UVs
	FQualityMetric QualityMetric;

	const uint8* RawElementData = (const uint8*)Stream;

	// Walk over UVs, note, we can get better results if we walk the indices and use knowledge on the triangles to predict the UVs
	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffsetBytes)
	{
		// Load data
		FVector3f& MVValue = *(FVector3f*)RawElementData;

		FIntVector Encoded = Quantizer.Quantize(MVValue);

		FIntVector Prediction = ReconstructedHistory[0]; // Delta coding
		FIntVector Residual = Encoded - Prediction;

		// Write residual	
		WriteInt32(EncodingContext.ResidualMotionVectorTable, Residual.X);
		WriteInt32(EncodingContext.ResidualMotionVectorTable, Residual.Y);
		WriteInt32(EncodingContext.ResidualMotionVectorTable, Residual.Z);

		// Store previous encountered values
		FIntVector Reconstructed = Prediction + Residual;
		ReconstructedHistory.Push(Reconstructed);

		// Calculate error
		FVector3f DequantReconstructed = Quantizer.Dequantize(Reconstructed);
		QualityMetric.Register(MVValue, DequantReconstructed);
	}

	// Gather rate and quality statistics
	Stats = FStreamEncodingStatistics(ByteCounter.Read(), ElementCount * sizeof(FVector2f), QualityMetric.ReadMSE());
}
void FCodecV1Encoder::WriteCodedStreamDescription()
{
	const FGeometryCacheVertexInfo& VertexInfo = EncodingContext.MeshData->VertexInfo;

	WriteBits(VertexInfo.bHasTangentX ? 1 : 0, 1);
	WriteBits(VertexInfo.bHasTangentZ ? 1 : 0, 1);
	WriteBits(VertexInfo.bHasUV0 ? 1 : 0, 1);
	WriteBits(VertexInfo.bHasColor0 ? 1 : 0, 1);
	WriteBits(VertexInfo.bHasMotionVectors ? 1 : 0, 1);
	WriteBits(VertexInfo.bHasImportedVertexNumbers ? 1 : 0, 1);

	WriteBits(VertexInfo.bConstantUV0 ? 1 : 0, 1);
	WriteBits(VertexInfo.bConstantColor0 ? 1 : 0, 1);
	WriteBits(VertexInfo.bConstantIndices ? 1 : 0, 1);
}

bool FCodecV1Encoder::EncodeFrameData(FMemoryWriter& Writer, const FGeometryCacheCodecEncodeArguments &Args)
{
/*
#ifdef DEBUG_DUMP_FRAMES
	{
		// Dump a frame to disk for codec development purposes
		static int32 FrameIndex = 0;
		FString Path = TEXT("E:\\");
		FString FileName = Path + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_raw.dump"); // F:\\frame_%i_raw.dump
		CodecV1Test::WriteRawMeshDataToFile(Args.MeshData, FileName);
		FrameIndex++;		
	}
#endif
*/
	const FGeometryCacheMeshData& MeshData = Args.MeshData;

	FExperimentTimer CodingTime;

	// Two-pass encoding: first we collect statistics and don't write any bits, second, we use the collected statistics and write our bitstream
	bool bPerformPrepass = true; // For now we always perform a prepass. In the future, we can e.g., do a prepass only at the start of a group-of-frames.
	if (bPerformPrepass)
	{
		// First pass, collect statistics
		bool bSuccess = EncodeFrameData(Writer, MeshData, /*bPrePass=*/true);
		if (!bSuccess)
		{
			return false;
		}
	}

	// Second pass, use statistics and actually write the bitstream
	bool bSuccess = EncodeFrameData(Writer, MeshData, /*bPrePass=*/false);
	if (!bSuccess)
	{
		return false;
	}

	// Additional stats
	Statistics.DurationMs = static_cast<float>(CodingTime.Get());
	Statistics.All.Quality = 0.0f;	
	Statistics.NumVertices = MeshData.Positions.Num();
	UE_LOGF(LogGeoCaStreamingCodecV1, Log, "Compressed %u vertices, %u bytes to %u bytes in %.2f milliseconds (%.2f ratio), quantizer precision: %.2f units.", 
		Statistics.NumVertices, Statistics.All.RawNumBytes, Statistics.All.CodedNumBytes, Statistics.DurationMs, Statistics.All.CompressionRatio, Config.VertexQuantizationPrecision);

	return true;
}

bool FCodecV1Encoder::EncodeFrameData(FMemoryWriter& Writer, const FGeometryCacheMeshData& MeshData, bool bPrepass)
{	
	FHuffmanBitStreamWriter BitWriter;

	EncodingContext.MeshData = &MeshData;
	EncodingContext.Writer = &BitWriter;
	EncodingContext.bPrepass = bPrepass;

	SetPrepass(bPrepass); // Tell our tables we are collecting or using statistics

	if (!bPrepass)
	{
		// Write in bitstream which streams are embedded
		WriteCodedStreamDescription();

		// Write tables on the second pass, when we are writing the bitstream
		WriteTables();
	}

	const TArray<FVector3f>& Positions = MeshData.Positions;
	const TArray<FVector2f>& TextureCoordinates = MeshData.TextureCoordinates;
	const TArray<FPackedNormal>& TangentsX = MeshData.TangentsX;
	const TArray<FPackedNormal>& TangentsZ = MeshData.TangentsZ;
	const TArray<FColor>& Colors = MeshData.Colors;
	const TArray<uint32>& ImportedVertexNumbers = MeshData.ImportedVertexNumbers;

	const TArray<uint32>& Indices = MeshData.Indices;
	const TArray<FVector3f>& MotionVectors = MeshData.MotionVectors;	

	const FGeometryCacheVertexInfo& VertexInfo = MeshData.VertexInfo;

	{
		// Check if indices are referenced in order, i.e. if a previously unreferenced vertex is referenced by the index 
		// list it's id will always be the next unreferenced id instead of some random unused id. E.g., ok: 1, 2, 3, 2, 4, not ok: 1, 2, 4
		// This is a requirement of the encoder and should be enforced by the preprocessor. 
		uint32 MaxIndex = 0;
		for (int32 i = 0; i < Indices.Num(); ++i)
		{
			bool bIsInOrder = Indices[i] <= MaxIndex + 1;
			checkf(bIsInOrder, TEXT("Vertices are not referenced in index buffer in order. Please make sure the preprocessor has processed the mesh such that vertexes are referenced in-order, i.e. if a previously unreferenced vertex is referenced by the index list it's id will always be the next unreferenced id instead of some random unused id."));
			MaxIndex = FMath::Max(Indices[i], MaxIndex);
		}
	}

	checkf(!VertexInfo.bHasMotionVectors || MotionVectors.Num() > 0, TEXT("No motion vectors while VertexInfo states otherwise"));

	FBitstreamWriterByteCounter TotalByteCounter(EncodingContext.Writer);
	
	const uint32 NumIndices = MeshData.Indices.Num();
	const uint32 NumVertices = MeshData.Positions.Num();

	const uint32 NumChunks = FMath::Max((NumIndices + MAX_CHUNK_INDICES - 1) / MAX_CHUNK_INDICES, (NumVertices + MAX_CHUNK_VERTICES - 1) / MAX_CHUNK_VERTICES);
	TArray<uint32> ChunkOffsets;
	ChunkOffsets.AddUninitialized(NumChunks);

	// Encode streams
	for (uint32 ChunkIndex = 0; ChunkIndex < NumChunks; ChunkIndex++)
	{
		uint32 IndexOffset = ChunkIndex * (NumIndices / NumChunks);
		uint32 VertexOffset = ChunkIndex * (NumVertices / NumChunks);
		uint32 NumChunkIndices = (ChunkIndex == NumChunks - 1) ? (NumIndices - IndexOffset) : (NumIndices / NumChunks);
		uint32 NumChunkVertices = (ChunkIndex == NumChunks - 1) ? (NumVertices - VertexOffset) : (NumVertices / NumChunks);

		EncodingContext.Writer->Align();
		ChunkOffsets[ChunkIndex] = EncodingContext.Writer->GetNumBytes();

		if (NumChunkIndices > 0 && !VertexInfo.bConstantIndices)
		{
			EncodeIndexStream(&Indices[IndexOffset], Indices.GetTypeSize(), NumChunkIndices, Statistics.Indices);
		}

		if (NumChunkVertices > 0)
		{
			EncodePositionStream(&Positions[VertexOffset], Positions.GetTypeSize(), NumChunkVertices, Statistics.Vertices);

			if (VertexInfo.bHasImportedVertexNumbers)
			{
				EncodeIndexStream(&ImportedVertexNumbers[VertexOffset], ImportedVertexNumbers.GetTypeSize(), NumChunkVertices, Statistics.ImportedVertexNumbers);
			}

			if (VertexInfo.bHasColor0)
			{
				EncodeColorStream(&Colors[VertexOffset], Colors.GetTypeSize(), NumChunkVertices, Statistics.Colors);
			}
			if (VertexInfo.bHasTangentX)
			{
				EncodeNormalStream(&TangentsX[VertexOffset], TangentsX.GetTypeSize(), NumChunkVertices, EncodingContext.ResidualNormalTangentXTable, Statistics.TangentX);
			}
			if (VertexInfo.bHasTangentZ)
			{
				EncodeNormalStream(&TangentsZ[VertexOffset], TangentsZ.GetTypeSize(), NumChunkVertices, EncodingContext.ResidualNormalTangentZTable, Statistics.TangentY);
			}
			if (VertexInfo.bHasUV0)
			{
				EncodeUVStream(&TextureCoordinates[VertexOffset], TextureCoordinates.GetTypeSize(), NumChunkVertices, Statistics.TexCoords);
			}
			if (VertexInfo.bHasMotionVectors)
			{
				EncodeMotionVectorStream(&MotionVectors[VertexOffset], MotionVectors.GetTypeSize(), NumChunkVertices, Statistics.MotionVectors);
			}
		}
	}
	
	BitWriter.Close();
	
	if (!bPrepass)
	{
		// Write out bitstream
		FCodedFrameHeader Header = { 0 };
		Header.Magic = V3_MAGIC;
		Header.VertexCount = (uint32)Positions.Num();
		Header.IndexCount = (uint32)Indices.Num();
		uint32 PayloadSize = BitWriter.GetNumBytes();
		Header.PayloadSize = PayloadSize;
		Writer.Serialize(&Header, sizeof(Header));	// Write header				
		Writer << MeshData.BatchesInfo;	// Uncompressed data: bounding box & material list
		Writer << MeshData.BoundingBox;
		Writer << ChunkOffsets;
		Writer.Serialize((void*)BitWriter.GetBytes().GetData(), PayloadSize);	// Write payload
	}	

	// Gather stats for all streams
	const uint32 TotalRawSize =
		sizeof(uint32) * Indices.Num()	// Indices
		+ sizeof(uint32) * ImportedVertexNumbers.Num()	// Imported vertex numbers
		+ sizeof(FVector3f) * Positions.Num() // Vertices
		+ sizeof(FColor) * Colors.Num() // Colors
		+ sizeof(FPackedNormal) * TangentsX.Num() // TangentX
		+ sizeof(FPackedNormal) * TangentsZ.Num() // TangentY
		+ sizeof(FVector2f) * TextureCoordinates.Num(); // UVs
	Statistics.All = FStreamEncodingStatistics(TotalByteCounter.Read() + sizeof(FCodedFrameHeader), TotalRawSize, 0.0f);
		
	return true;
}


void FCodecV1Encoder::WriteBytes(const void* Data, int64 NumBytes)
{
	if (EncodingContext.bPrepass)
	{
		return; // Nothing gets actually written in the prepass phase
	}

	const uint8* ByteData = (const uint8*)Data;
	
	for (int64 ByteIndex = 0; ByteIndex < NumBytes; ++ByteIndex)
	{
		uint32 ByteValue = *ByteData++;
		EncodingContext.Writer->Write(ByteValue, 8);
	}
}


void FCodecV1Encoder::WriteInt32(FHuffmanEncodeTable& ValueTable, int32 Value)
{
	// It is impractical to entropy code an entire integer, so we split it into an entropy coded magnitude followed by a number of raw bits.
	// The reasoning is that usually most of the redundancy is in the magnitude of the number, not the exact value.
	
	// Positive values are encoded as the index k of the first 1-bit (at most 30) followed by the remaining k bits encoded as raw bits.
	// Negative values are handled symmetrically, but using the index of the first 0-bit.
	// With one symbol for every bit length and sign, the set of reachable number is 2 * (2^0 + 2^1 + ... + 2^30) = 2 * (2^31 - 1) = 2^32 - 2
	// To cover all 2^32 possible integer values, we have use separate codes for the remaining two symbols (with no raw bits).
	// The total number of symbols is 2 * 31 + 2 = 64
	
	if (Value >= -2 && Value <= 1)
	{
		// 4 center values have no raw bits. One more negative values than positive,
		// so we have an equal number of positive and negative values remaining.
		WriteSymbol(ValueTable, Value + 2);	// [-2, 1] -> [0, 3]
	}
	else
	{
		// At least one raw bit.
		if (Value >= 0)
		{
			// Value >= 2
			int32 NumRawBits = HighestSetBit(Value);	// Find first 1-bit. 1 <= NumRawBits <= 30.
			int32 Packed = 2 + NumRawBits * 2;			// First positive code is 4
			WriteSymbol(ValueTable, Packed);
			int32 RawBits = Value - (1 << NumRawBits);
			WriteBits(RawBits, NumRawBits);
		}
		else
		{
			// Value <= -3
			int32 NumRawBits = HighestSetBit(~Value);	// Find first 0-bit. 1 <= NumRawBits <= 30.
			int32 Packed = 3 + NumRawBits * 2;			// First negative code is 5
			WriteSymbol(ValueTable, Packed);
			int32 RawBits = Value & ~(0xFFFFFFFFu << NumRawBits);
			WriteBits(RawBits, NumRawBits);
		}
	}
	
}

void FCodecV1Encoder::SetupTables()
{
	// Initialize Huffman tables.
	// Most tables store 32-bit integers stored with a bit-length;raw value scheme. Some store specific symbols.
	EncodingContext.ResidualIndicesTable.Initialize(HuffmanTableInt32SymbolCount);
	EncodingContext.ResidualVertexPosTable.Initialize(HuffmanTableInt32SymbolCount);
	EncodingContext.ResidualImportedVertexNumbersTable.Initialize(HuffmanTableInt32SymbolCount);
	EncodingContext.ResidualColorTable.Initialize(HuffmanTableInt32SymbolCount);
	EncodingContext.ResidualNormalTangentXTable.Initialize(HuffmanTableInt8SymbolCount);
	EncodingContext.ResidualNormalTangentZTable.Initialize(HuffmanTableInt8SymbolCount);
	EncodingContext.ResidualUVTable.Initialize(HuffmanTableInt32SymbolCount);
	EncodingContext.ResidualMotionVectorTable.Initialize(HuffmanTableInt32SymbolCount);
	// Add additional tables here
}

void FCodecV1Encoder::SetPrepass(bool bPrepass)
{
	const FGeometryCacheVertexInfo& VertexInfo = EncodingContext.MeshData->VertexInfo;

	// When bPrepass is set to true, the tables gather statistics about the data they encounter and do not write 
	// any output bits.When set to false, they build the internal symbol representations and will write bits.
	if (!VertexInfo.bConstantIndices)
	{
		EncodingContext.ResidualIndicesTable.SetPrepass(bPrepass);
	}

	EncodingContext.ResidualVertexPosTable.SetPrepass(bPrepass);

	if (VertexInfo.bHasImportedVertexNumbers)
	{
		EncodingContext.ResidualImportedVertexNumbersTable.SetPrepass(bPrepass);
	}
	if (VertexInfo.bHasColor0)
	{
		EncodingContext.ResidualColorTable.SetPrepass(bPrepass);
	}
	if (VertexInfo.bHasTangentX)
	{
		EncodingContext.ResidualNormalTangentXTable.SetPrepass(bPrepass);
	}
	if (VertexInfo.bHasTangentZ)
	{
		EncodingContext.ResidualNormalTangentZTable.SetPrepass(bPrepass);
	}
	if (VertexInfo.bHasUV0)
	{
		EncodingContext.ResidualUVTable.SetPrepass(bPrepass);
	}
	if (VertexInfo.bHasMotionVectors)
	{
		EncodingContext.ResidualMotionVectorTable.SetPrepass(bPrepass);
	}
	
	// Add additional tables here
}

void FCodecV1Encoder::WriteTables()
{
	// Write all our Huffman tables to the bitstream. This gets typically done after a SetPrepass(false) call sets
	// up the tables for their first use, and before symbols are written.
	FBitstreamWriterByteCounter ByteCounter(EncodingContext.Writer); // Count the bytes we are going to write
	FHuffmanBitStreamWriter& Writer = *EncodingContext.Writer;	
	const FGeometryCacheVertexInfo& VertexInfo = EncodingContext.MeshData->VertexInfo;

	if (!VertexInfo.bConstantIndices)
	{
		EncodingContext.ResidualIndicesTable.Serialize(Writer);
	}

	EncodingContext.ResidualVertexPosTable.Serialize(Writer);

	if (VertexInfo.bHasImportedVertexNumbers)
	{
		EncodingContext.ResidualImportedVertexNumbersTable.Serialize(Writer);
	}
	if (VertexInfo.bHasColor0)
	{
		EncodingContext.ResidualColorTable.Serialize(Writer);
	}
	if (VertexInfo.bHasTangentX)
	{
		EncodingContext.ResidualNormalTangentXTable.Serialize(Writer);
	}
	if (VertexInfo.bHasTangentZ)
	{
		EncodingContext.ResidualNormalTangentZTable.Serialize(Writer);
	}
	if (VertexInfo.bHasUV0)
	{
		EncodingContext.ResidualUVTable.Serialize(Writer);
	}	
	if (VertexInfo.bHasMotionVectors)
	{
		EncodingContext.ResidualMotionVectorTable.Serialize(Writer);
	}
	// Add additional tables here

	Statistics.HuffmanTablesNumBytes = ByteCounter.Read();
}

#endif // WITH_EDITOR

void FCodecV1Decoder::DecodeIndexStream(FHuffmanBitStreamReader& Reader, uint32* Stream, uint64 ElementOffset, uint32 ElementCount)
{
	const uint8* RawElementData = (const uint8*)Stream;

	uint32 Value = 0;
	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffset)
	{
		// Read coded residual
		int32 DecodedResidual = ReadInt32(Reader, DecodingContext.ResidualIndicesTable);
		Value += DecodedResidual;

		// Save result to our list
		*(uint32*)RawElementData = Value;
	}
}

void FCodecV1Decoder::DecodeMotionVectorStream(FHuffmanBitStreamReader& Reader, FVector3f* Stream, uint64 ElementOffset, uint32 ElementCount)
{
	// Read header
	FMotionVectorStreamHeader Header;
	ReadBytes(Reader, (void*)&Header, sizeof(Header));

	FQuantizerVector3 Quantizer(Header.QuantizationPrecision); // We quantize MVs to a certain precision just like the positions

	FIntVector QuantizedValue(0, 0, 0);

	const uint8* RawElementData = (const uint8*)Stream;

	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffset) // Walk MVs
	{
		// Read coded residual
		FIntVector DecodedResidual;
		DecodedResidual.X = ReadInt32(Reader, DecodingContext.ResidualMotionVectorTable);
		DecodedResidual.Y = ReadInt32(Reader, DecodingContext.ResidualMotionVectorTable);
		DecodedResidual.Z = ReadInt32(Reader, DecodingContext.ResidualMotionVectorTable);

		QuantizedValue += DecodedResidual;
		*(FVector3f*)RawElementData = Quantizer.Dequantize(QuantizedValue);
	}
}

void FCodecV1Decoder::DecodeUVStream(FHuffmanBitStreamReader& Reader, FVector2f* Stream, uint64 ElementOffset, uint32 ElementCount)
{
	// Read header
	FUVStreamHeader Header;
	ReadBytes(Reader, (void*)&Header, sizeof(Header));

	FQuantizerVector2 Quantizer(Header.Range, Header.QuantizationBits); // We quantize UVs to a number of bits, set in the bitstream header
	
	FIntVector QuantizedValue(0, 0, 0);

	const uint8* RawElementData = (const uint8*)Stream;

	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffset) // Walk UVs
	{
		// Read coded residual
		FIntVector DecodedResidual;
		DecodedResidual.X = ReadInt32(Reader, DecodingContext.ResidualUVTable);
		DecodedResidual.Y = ReadInt32(Reader, DecodingContext.ResidualUVTable);

		QuantizedValue += DecodedResidual;
		*(FVector2f*)RawElementData = Quantizer.Dequantize(QuantizedValue);
	}
}

void FCodecV1Decoder::DecodeNormalStream(FHuffmanBitStreamReader& Reader, FPackedNormal* Stream, uint64 ElementOffset, uint32 ElementCount, FHuffmanDecodeTable& Table)
{
	uint8 x = 128, y = 128, z = 128, w = 128;

	const uint8* RawElementData = (const uint8*)Stream;

	check(HUFFMAN_MAX_CODE_LENGTH * 4 <= MINIMUM_BITS_AFTER_REFILL);	// Make sure we can safely decode all 4 symbols with a single refill

	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffset) // Walk normals
	{
		// Read coded residual
		Reader.Refill();
		x += static_cast<uint8>(Table.DecodeNoRefill(Reader));	
		y += static_cast<uint8>(Table.DecodeNoRefill(Reader));
		z += static_cast<uint8>(Table.DecodeNoRefill(Reader));
		w += static_cast<uint8>(Table.DecodeNoRefill(Reader));
		
		FPackedNormal* Value = (FPackedNormal*)RawElementData;
		Value->Vector.X = x;
		Value->Vector.Y = y;
		Value->Vector.Z = z;
		Value->Vector.W = w;
	}
}

void FCodecV1Decoder::DecodeColorStream(FHuffmanBitStreamReader& Reader, FColor* Stream, uint64 ElementOffset, uint32 ElementCount)
{
	FIntVector4 QuantizedValue(128, 128, 128, 255);

	const uint8* RawElementData = (const uint8*)Stream;

	for (uint32 ElementIdx = 0; ElementIdx < ElementCount; ++ElementIdx, RawElementData += ElementOffset)
	{
		FIntVector4 DecodedResidual(0, 0, 0, 0);

		int32 SkipBit = ReadBits(Reader, 1); // 1: Perfect prediction, nothing coded, 0: we have coded residuals

		if (SkipBit != 1) 
		{
			// Prediction not perfect, residual were coded
			int32 DecodedResidualR = ReadInt32(Reader, DecodingContext.ResidualColorTable);
			int32 DecodedResidualG = ReadInt32(Reader, DecodingContext.ResidualColorTable);
			int32 DecodedResidualB = ReadInt32(Reader, DecodingContext.ResidualColorTable);
			int32 DecodedResidualA = ReadInt32(Reader, DecodingContext.ResidualColorTable);

			DecodedResidual = FIntVector4(DecodedResidualR, DecodedResidualG, DecodedResidualB, DecodedResidualA);
			QuantizedValue = FCodecV1SharedTools::SumVector4(QuantizedValue, DecodedResidual);
		}
																			
		FColor* Value = (FColor*)RawElementData; // Save result to our list
		Value->R = static_cast<uint8>(QuantizedValue.X);
		Value->G = static_cast<uint8>(QuantizedValue.Y);
		Value->B = static_cast<uint8>(QuantizedValue.Z);
		Value->A = static_cast<uint8>(QuantizedValue.W);
	}
}

void FCodecV1Decoder::DecodePositionStream(FHuffmanBitStreamReader& Reader, FVector3f* VertexStream, uint64 VertexElementOffset, uint32 VertexElementCount)
{
	// Read header
	FVertexStreamHeader Header;
	ReadBytes(Reader, (void*)&Header, sizeof(Header));

	FQuantizerVector3 Quantizer(Header.QuantizationPrecision);
	
	uint32 DecodedVertexCount = 0;

	const uint8* RawElementDataVertices = (const uint8*)VertexStream;

	FIntVector QuantizedValue(0, 0, 0);

	// Walk over indices/triangles
	for (uint32 ElementIdx = 0; ElementIdx < VertexElementCount; ++ElementIdx)
	{
		// Read coded residual
		const FIntVector DecodedResidual = { ReadInt32(Reader, DecodingContext.ResidualVertexPosTable), ReadInt32(Reader, DecodingContext.ResidualVertexPosTable), ReadInt32(Reader, DecodingContext.ResidualVertexPosTable) };
		DecodedVertexCount++;

		QuantizedValue += DecodedResidual;

		// Save result to our list
		FVector3f* Value = (FVector3f*)RawElementDataVertices;
		*Value = Quantizer.Dequantize(QuantizedValue + Header.Translation);
		RawElementDataVertices += VertexElementOffset;
	}
}

void FCodecV1Decoder::SetupAndReadTables(FHuffmanBitStreamReader& Reader)
{
	// Initialize and read Huffman tables from the bitstream
	const FGeometryCacheVertexInfo& VertexInfo = DecodingContext.MeshData->VertexInfo;
		
	if (!VertexInfo.bConstantIndices)
	{
		DecodingContext.ResidualIndicesTable.Initialize(Reader);
	}

	DecodingContext.ResidualVertexPosTable.Initialize(Reader);

	if (VertexInfo.bHasImportedVertexNumbers)
	{
		DecodingContext.ResidualImportedVertexNumbersTable.Initialize(Reader);
	}
	if (VertexInfo.bHasColor0)
	{
		DecodingContext.ResidualColorTable.Initialize(Reader);		
	}
	if (VertexInfo.bHasTangentX)
	{
		DecodingContext.ResidualNormalTangentXTable.Initialize(Reader);
	}
	if (VertexInfo.bHasTangentZ)
	{
		DecodingContext.ResidualNormalTangentZTable.Initialize(Reader);
	}
	if (VertexInfo.bHasUV0)
	{
		DecodingContext.ResidualUVTable.Initialize(Reader);
	}
	if (VertexInfo.bHasMotionVectors)
	{
		DecodingContext.ResidualMotionVectorTable.Initialize(Reader);
	}
	
	// Add additional tables here
}

void FCodecV1Decoder::ReadCodedStreamDescription(FHuffmanBitStreamReader& Reader, uint32 Version)
{	
	FGeometryCacheVertexInfo& VertexInfo = DecodingContext.MeshData->VertexInfo;
	
	VertexInfo = FGeometryCacheVertexInfo();
	VertexInfo.bHasTangentX = (ReadBits(Reader, 1) == 1);
	VertexInfo.bHasTangentZ = (ReadBits(Reader, 1) == 1);
	VertexInfo.bHasUV0 = (ReadBits(Reader, 1) == 1);
	VertexInfo.bHasColor0 = (ReadBits(Reader, 1) == 1);
	VertexInfo.bHasMotionVectors = (ReadBits(Reader, 1) == 1);
	if (Version >= V3_MAGIC)
	{
		VertexInfo.bHasImportedVertexNumbers = (ReadBits(Reader, 1) == 1);
	}

	VertexInfo.bConstantUV0 = (ReadBits(Reader, 1) == 1);
	VertexInfo.bConstantColor0 = (ReadBits(Reader, 1) == 1);
	VertexInfo.bConstantIndices = (ReadBits(Reader, 1) == 1);
}

int32 FCodecV1Decoder::CachedHighBitsLUT[64];
void FCodecV1Decoder::InitLUT()
{
	// Precalculate table mapping symbol index to non-raw bits. ((Sign ? -2 : 1) << NumRawBits)
	for (int32 NumRawBits = 1; NumRawBits <= 30; NumRawBits++)
	{
		for (int32 Sign = 0; Sign <= 1; Sign++)
		{
			CachedHighBitsLUT[2 + Sign + NumRawBits * 2] = int32(uint32(Sign ? -2 : 1) << NumRawBits);
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("FCodecV1Decoder"), STAT_CodecV1Decoder, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("SetupAndReadTables"), STAT_SetupAndReadTables, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeIndexStream"), STAT_DecodeIndexStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeImportedVertexNumbersStream"), STAT_DecodeImportedVertexNumbersStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodePositionStream"), STAT_DecodePositionStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeColorStream"), STAT_DecodeColorStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeTangentXStream"), STAT_DecodeTangentXStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeTangentZStream"), STAT_DecodeTangentZStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeUVStream"), STAT_DecodeUVStream, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("DecodeMotionVectorStream"), STAT_DecodeMotionVectorStream, STATGROUP_GeometryCache);


bool FCodecV1Decoder::DecodeFrameData(FBufferReader& Reader, FGeometryCacheMeshData &OutMeshData)
{
	FExperimentTimer DecodingTime;

	// Read stream header
	FCodedFrameHeader Header;
	Reader.Serialize(&Header, sizeof(Header));

	if (Header.Magic != V1_MAGIC && Header.Magic != V2_MAGIC && Header.Magic != V3_MAGIC)
	{
		UE_LOGF(LogGeoCaStreamingCodecV1, Error, "Incompatible bitstream found");
		return false;
	}
		
	// Read uncompressed data: bounding box & material list
	Reader << OutMeshData.BatchesInfo; 
	Reader << OutMeshData.BoundingBox;

	const uint32 NumIndices = Header.IndexCount;
	const uint32 NumVertices = Header.VertexCount;

	uint32 NumChunks;
	TArray<uint32, TInlineAllocator<8>> ChunkOffsets;

	bool IsChunked;
	if (Header.Magic == V1_MAGIC)
	{
		// Compatibility with old files. Just treat everything as one big chunk
		IsChunked = false;
		NumChunks = 1;
	}
	else
	{
		check(Header.Magic == V2_MAGIC || Header.Magic == V3_MAGIC);
		IsChunked = true;
		Reader << ChunkOffsets;
		NumChunks = ChunkOffsets.Num();
	}
	
	DecodingContext = { 0 };
	DecodingContext.MeshData = &OutMeshData;	
	
	// Read the payload in memory to pass to the bit reader
	TArray<uint8>& Bytes = OutMeshData.TempBytes;
	Bytes.SetNumUninitialized(Header.PayloadSize + 16, EAllowShrinking::No); // Overallocate by 16 bytes to ensure BitReader can safely perform uint64 reads.
	Reader.Serialize(Bytes.GetData(), Header.PayloadSize);
	FHuffmanBitStreamReader BitReader(Bytes.GetData(), Bytes.Num());
	
	// Read which vertex attributes are in the bit stream
	ReadCodedStreamDescription(BitReader, Header.Magic);

	// Restore entropy coding contexts
	{
		SCOPE_CYCLE_COUNTER(STAT_SetupAndReadTables);
		SetupAndReadTables(BitReader);
	}
	
	// Allocate buffers
	{
		if (!OutMeshData.VertexInfo.bConstantIndices)
		{
			OutMeshData.Indices.SetNumUninitialized(Header.IndexCount, EAllowShrinking::No);
		}

		OutMeshData.Positions.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);

		OutMeshData.ImportedVertexNumbers.Reset(Header.VertexCount);
		if (OutMeshData.VertexInfo.bHasImportedVertexNumbers)
		{
			OutMeshData.ImportedVertexNumbers.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);
		}

		if (OutMeshData.VertexInfo.bHasColor0)
		{
			OutMeshData.Colors.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);
		}
		else
		{
			OutMeshData.Colors.SetNumZeroed(Header.VertexCount, EAllowShrinking::No);
		}	
		
		if (OutMeshData.VertexInfo.bHasTangentX)
		{
			OutMeshData.TangentsX.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);
		}
		else
		{
			OutMeshData.TangentsX.SetNumZeroed(Header.VertexCount, EAllowShrinking::No);
		}
		
		if (OutMeshData.VertexInfo.bHasTangentZ)
		{
			OutMeshData.TangentsZ.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);
		}
		else
		{
			OutMeshData.TangentsZ.SetNumZeroed(Header.VertexCount, EAllowShrinking::No);
		}
		
		if (OutMeshData.VertexInfo.bHasUV0)
		{
			OutMeshData.TextureCoordinates.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);
		}
		else
		{
			OutMeshData.TextureCoordinates.SetNumZeroed(Header.VertexCount, EAllowShrinking::No);
		}	
		
		if (OutMeshData.VertexInfo.bHasMotionVectors)
		{
			OutMeshData.MotionVectors.SetNumUninitialized(Header.VertexCount, EAllowShrinking::No);
		}
		else
		{
			OutMeshData.MotionVectors.SetNumZeroed(Header.VertexCount, EAllowShrinking::No);
		}
	}
	
	// Decompress chunks using the task graph. Note: ParallelFor is slow to wake threads. It can take up to 1ms!
	// TODO: can we sync later to get more overlap?
	ParallelFor(NumChunks, [this, NumChunks, NumIndices, NumVertices, IsChunked, &Bytes, &BitReader, &ChunkOffsets, &OutMeshData](int32 ChunkIndex)
	{
		SCOPE_CYCLE_COUNTER(STAT_CodecV1Decoder);

		uint32 IndexOffset = 0;
		uint32 VertexOffset = 0;
		uint32 NumChunkIndices = NumIndices;
		uint32 NumChunkVertices = NumVertices;

		uint32 ChunkOffset = 0;
		if (IsChunked)
		{
			IndexOffset = (uint32)ChunkIndex * (NumIndices / NumChunks);
			VertexOffset = (uint32)ChunkIndex * (NumVertices / NumChunks);
			NumChunkIndices = (ChunkIndex + 1 == NumChunks) ? (NumIndices - IndexOffset) : (NumIndices / NumChunks);
			NumChunkVertices = (ChunkIndex + 1 == NumChunks) ? (NumVertices - VertexOffset) : (NumVertices / NumChunks);
			ChunkOffset = ChunkOffsets[ChunkIndex];
		}

		FHuffmanBitStreamReader ReaderObj(Bytes.GetData() + ChunkOffset, Bytes.Num() - ChunkOffset);
		FHuffmanBitStreamReader& ChunkReader = IsChunked ? ReaderObj : BitReader;

		if (NumChunkIndices > 0 && !OutMeshData.VertexInfo.bConstantIndices)
		{
			SCOPE_CYCLE_COUNTER(STAT_DecodeIndexStream);
			DecodeIndexStream(ChunkReader, &OutMeshData.Indices[IndexOffset], OutMeshData.Indices.GetTypeSize(), NumChunkIndices);
		}

		if (NumChunkVertices > 0)
		{
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodePositionStream);
				DecodePositionStream(ChunkReader, &OutMeshData.Positions[VertexOffset], OutMeshData.Positions.GetTypeSize(), NumChunkVertices);
			}

			if (OutMeshData.VertexInfo.bHasImportedVertexNumbers)
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodeImportedVertexNumbersStream);
				DecodeIndexStream(ChunkReader, &OutMeshData.ImportedVertexNumbers[VertexOffset], OutMeshData.ImportedVertexNumbers.GetTypeSize(), NumChunkVertices);
			}

			if (OutMeshData.VertexInfo.bHasColor0)
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodeColorStream);
				DecodeColorStream(ChunkReader, &OutMeshData.Colors[VertexOffset], OutMeshData.Colors.GetTypeSize(), NumChunkVertices);
			}

			if (OutMeshData.VertexInfo.bHasTangentX)
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodeTangentXStream);
				DecodeNormalStream(ChunkReader, &OutMeshData.TangentsX[VertexOffset], OutMeshData.TangentsX.GetTypeSize(), NumChunkVertices, DecodingContext.ResidualNormalTangentXTable);
			}

			if (OutMeshData.VertexInfo.bHasTangentZ)
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodeTangentZStream);
				DecodeNormalStream(ChunkReader, &OutMeshData.TangentsZ[VertexOffset], OutMeshData.TangentsZ.GetTypeSize(), NumChunkVertices, DecodingContext.ResidualNormalTangentZTable);
			}

			if (OutMeshData.VertexInfo.bHasUV0)
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodeUVStream);
				DecodeUVStream(ChunkReader, &OutMeshData.TextureCoordinates[VertexOffset], OutMeshData.TextureCoordinates.GetTypeSize(), NumChunkVertices);
			}

			if (OutMeshData.VertexInfo.bHasMotionVectors)
			{
				SCOPE_CYCLE_COUNTER(STAT_DecodeMotionVectorStream);
				DecodeMotionVectorStream(ChunkReader, &OutMeshData.MotionVectors[VertexOffset], OutMeshData.MotionVectors.GetTypeSize(), NumChunkVertices);
			}
		}
	});

	if (CVarCodecDebug.GetValueOnAnyThread() == 1)
	{
		const float TimeFloat = static_cast<float>(DecodingTime.Get());
		UE_LOGF(LogGeoCaStreamingCodecV1, Log, "Decoded frame with %i vertices in %.2f milliseconds.", NumVertices, TimeFloat);
	}
	
	return true;
}

void FCodecV1Decoder::ReadBytes(FHuffmanBitStreamReader& Reader, void* Data, uint32 NumBytes)
{
	uint8* ByteData = (uint8*)Data;

	for (int64 ByteIndex = 0; ByteIndex < NumBytes; ++ByteIndex)
	{
		const uint32 ByteValue = Reader.Read(8);
		*ByteData++ = ByteValue & 255;
	}
}

int32 FCodecV1Decoder::ReadInt32(FHuffmanBitStreamReader& Reader, FHuffmanDecodeTable& ValueTable)
{
	// See write WriteInt32 for encoding details.
	const int32 Packed = ReadSymbol(Reader, ValueTable);
	if (Packed < 4)
	{
		// [-2, 1] coded directly with no additional raw bits
		return Packed - 2;
	}
	else
	{
		// At least one raw bit.
		int32 NumRawBits = (Packed - 2) >> 1;
		return ReadBitsNoRefill(Reader, NumRawBits) + CachedHighBitsLUT[Packed];
	}
}
