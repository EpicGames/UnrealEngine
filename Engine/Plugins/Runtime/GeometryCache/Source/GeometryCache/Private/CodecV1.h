// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ICodecEncoder.h"
#include "ICodecDecoder.h"
#include "HuffmanTable.h"
#include "Math/Vector.h"

struct FPackedNormal;
template <class T, size_t N> class FRingBuffer;

DECLARE_LOG_CATEGORY_EXTERN(LogGeoCaStreamingCodecV1, Verbose, All);

/** Quality metric to calculate Mean Square Error between 2D vectors */
class FQualityMetric2D
{
public:

	FQualityMetric2D() : SumSquaredX(0.0), SumSquaredY(0.0), Num(0)
	{
	}

	FORCEINLINE void Register(const FVector2f& Coded, const FVector2f& Original)
	{
		SumSquaredX += (Original.X - Coded.X)*(Original.X - Coded.X);
		SumSquaredY += (Original.Y - Coded.Y)*(Original.Y - Coded.Y);		
		Num++;
	}

	// Mean Squared Error
	FORCEINLINE float ReadMSE()
	{
		FVector2f MSE(static_cast<float>(SumSquaredX / static_cast<double>(Num)), static_cast<float>(SumSquaredY / static_cast<double>(Num)));
		return (MSE.X + MSE.Y) / 2.0f;
	}

private:
	double SumSquaredX;
	double SumSquaredY;	
	uint64 Num;
};

/** Quality metric to calculate Mean Square Error between vectors */
class FQualityMetric
{
public:

	FQualityMetric() : SumSquaredX(0.0), SumSquaredY(0.0), SumSquaredZ(0.0), Num(0)
	{
	}

	FORCEINLINE void Register(const FVector3f& Coded, const FVector3f& Original)
	{
		SumSquaredX += (Original.X - Coded.X)*(Original.X - Coded.X);
		SumSquaredY += (Original.Y - Coded.Y)*(Original.Y - Coded.Y);
		SumSquaredZ += (Original.Z - Coded.Z)*(Original.Z - Coded.Z);
		Num++;
	}

	// Mean Squared Error
	FORCEINLINE float ReadMSE()
	{
		FVector3f MSE(static_cast<float>(SumSquaredX / static_cast<double>(Num)), static_cast<float>(SumSquaredY / static_cast<double>(Num)), static_cast<float>(SumSquaredZ / static_cast<double>(Num)));
		return (MSE.X + MSE.Y + MSE.Z) / 3.0f;
	}

private:
	double SumSquaredX;
	double SumSquaredY;
	double SumSquaredZ;
	uint64 Num;
};

class FHuffmanBitStreamReader;
class FHuffmanBitStreamWriter;

/** Any context information to decode a frame in a sequence of frames */
struct FCodecV1DecodingContext
{
	/** Target mesh to decode to */
	FGeometryCacheMeshData* MeshData;	
	/** Huffman tables */
	FHuffmanDecodeTable ResidualIndicesTable;	
	FHuffmanDecodeTable ResidualVertexPosTable;
	FHuffmanDecodeTable ResidualImportedVertexNumbersTable;
	FHuffmanDecodeTable ResidualColorTable;
	FHuffmanDecodeTable ResidualNormalTangentXTable;
	FHuffmanDecodeTable ResidualNormalTangentZTable;
	FHuffmanDecodeTable ResidualUVTable;
	FHuffmanDecodeTable ResidualMotionVectorTable;
};

/** Configuration settings for the encoder */
struct FCodecV1EncoderConfig
{
	/** Vertex quantization precision. 
	    Each vertex' position is quantized with bin sizes equal to this size, i.e., the full range of the vertex positions is discretized 
		with steps equal in size of this value. E.g., steps 0.1 corresponds to bin sizes of 1 cubic millimeter as 1 unit equals 1 cm. 
		The lower this number, the higher the precision of the vertex positions, but the lower the compression ratio becomes. */
	float VertexQuantizationPrecision;
	
	/** Texture coordinate quantization bit range (for the fractional part). 
	    Each vertex' texture coordinate is quantized to this bit range, e.g., set to 10 bit, the range [0-1] is mapped to 1024 equal bins. 
		Any range outside [0-1] will be quantized with 'UVQuantizationBitRange' bits for the fraction part, and will need use extra bits for
		the non-fraction part, e.g., UDIM range [0-6] will use 10+3 bits instead. */
	int32 UVQuantizationBitRange;

	static FCodecV1EncoderConfig DefaultConfig()
	{
		FCodecV1EncoderConfig Config;		
		Config.UVQuantizationBitRange = 10;		// Fixed 10 bit quantization for UVs
		Config.VertexQuantizationPrecision = 0.01f; // Variable bit rate quantization, e.g., 1 bin per 0.1f^3 cubic units (0.1mm^3)
		return Config;
	}
};

/** Any context information to code a frame in a sequence of frames */
struct FCodecV1EncodingContext
{
	/** Are we in prepass mode, we do not write anything in prepass mode, only calculate statistics */
	bool bPrepass;
	/** Mesh to code */
	const FGeometryCacheMeshData* MeshData;	
	/** Huffman bit writer to write bitstream to */
	FHuffmanBitStreamWriter* Writer;	
	/** Huffman tables */
	FHuffmanEncodeTable ResidualIndicesTable;
	FHuffmanEncodeTable ResidualVertexPosTable;
	FHuffmanEncodeTable ResidualImportedVertexNumbersTable;
	FHuffmanEncodeTable ResidualColorTable;
	FHuffmanEncodeTable ResidualColorSkipTable;
	FHuffmanEncodeTable ResidualNormalTangentXTable;
	FHuffmanEncodeTable ResidualNormalTangentZTable;
	FHuffmanEncodeTable ResidualUVTable;
	FHuffmanEncodeTable ResidualMotionVectorTable;
};

const int32 HuffmanTableInt32SymbolCount = 64; // 31 negative lengths, zero, 32 positive length
const int32 HuffmanTableInt8SymbolCount = 256;
const uint32 VertexStreamCodingIndexHistorySize = 9; // Sizes of the previously-seen histories used for prediction of the various stream elements
const uint32 VertexStreamCodingVertexHistorySize = 9;
const uint32 IndexStreamCodingHistorySize = 5;
const uint32 ColorStreamCodingHistorySize = 5;
const uint32 NormalStreamCodingHistorySize = 5;
const uint32 UVStreamCodingHistorySize = 9;
const uint32 MotionVectorStreamCodingHistorySize = 9;

/** Shared functionality between encoder and decoder */
class FCodecV1SharedTools 
{
public:
	/** Sum two IntVector4 vectors, because IntVector4 does not implement arithmetic operations contrary to IntVector */
	static FORCEINLINE FIntVector4 SumVector4(const FIntVector4& First, const FIntVector4& Second)
	{
		return FIntVector4(First.X + Second.X, First.Y + Second.Y, First.Z + Second.Z, First.W + Second.W);
	}
	
	/** Subtract two IntVector4 vectors, because IntVector4 does not implement arithmetic operations contrary to IntVector */
	static FORCEINLINE FIntVector4 SubtractVector4(const FIntVector4& First, const FIntVector4& Second)
	{		
		return FIntVector4(First.X - Second.X, First.Y - Second.Y, First.Z - Second.Z, First.W - Second.W);
	}
};

/** Statistics on encoding of a single stream/buffer */
struct FStreamEncodingStatistics
{
	uint32 CodedNumBytes;
	uint32 RawNumBytes;
	float CompressionRatio;
	float Quality;

	FStreamEncodingStatistics() : CodedNumBytes(0), RawNumBytes(0), CompressionRatio(0.0f), Quality(0.0f) {}

	FStreamEncodingStatistics(uint32 SetCodedNumBytes, uint32 SetRawNumBytes, float SetQuality) : CodedNumBytes(SetCodedNumBytes), RawNumBytes(SetRawNumBytes), Quality(SetQuality)
	{
		CompressionRatio = static_cast<float>(RawNumBytes) / static_cast<float>(CodedNumBytes);
	}
};

#if WITH_EDITOR
class FCodecV1Encoder : public ICodecEncoder
{
public:
	FCodecV1Encoder();
	FCodecV1Encoder(const FCodecV1EncoderConfig& Config);

	/**
	* Encode a frame and write the bitstream
	*
	* @param Writer - Writer for the output bit stream
	* @param Args - Encoding arguments
	*/
	bool EncodeFrameData(FMemoryWriter& Writer, const FGeometryCacheCodecEncodeArguments &Args);

private:	
	/** Encode a single frame in or not in prepass mode and write to the bitstream */
	bool EncodeFrameData(FMemoryWriter &Writer, const FGeometryCacheMeshData& MeshData, bool bPrepass);

	/** Encode a buffer with vertex indices */
	void EncodeIndexStream(const uint32* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats);
	/** Encode a buffer with vertex positions */
	void EncodePositionStream(const FVector3f* VertexStream, uint64 VertexElementOffset, uint32 VertexElementCount, FStreamEncodingStatistics& Stats);
	/** Encode a buffer with vertex colors */
	void EncodeColorStream(const FColor* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats);
	/** Encode a buffer with vertex normals */
	void EncodeNormalStream(const FPackedNormal* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FHuffmanEncodeTable& Table, FStreamEncodingStatistics& Stats);
	/** Encode a buffer with vertex texture coordinates */
	void EncodeUVStream(const FVector2f* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats);
	/** Encode a buffer with vertex motion vectors */
	void EncodeMotionVectorStream(const FVector3f* Stream, uint64 ElementOffsetBytes, uint32 ElementCount, FStreamEncodingStatistics& Stats);

	/** Find the best prediction mode for a vertex position and predict according to that mode */
	FORCEINLINE FIntVector FindModeAndPredictVertex(const FIntVector& ValueToCode, uint32 CornerIndex, uint32 PreviousTriangleRotationOffsets[3], FRingBuffer<FIntVector, VertexStreamCodingVertexHistorySize>& ReconstructedVertexHistory, uint32& ChosenMode);
	
	/** Setup and write the Huffman coding tables */
	void SetupTables();
	void WriteTables();
	
	/** Set the prepass mode to Huffman tables and any other internal state */
	void SetPrepass(bool bPrepass);
	
	/** Write a given number of bytes to the bit stream using the Huffman bit writer	*/
	void WriteBytes(const void* Data, int64 Num);
	
	/**	Write a 32-bit signed value from the bit stream using a Huffman coder */
	void WriteInt32(FHuffmanEncodeTable& ValueTable, int32 Value);

	/** Write a given number of bits to the bit stream using the Huffman bit writer */
	FORCEINLINE void WriteBits(int32 Data, uint32 NumBits)
	{
		if (EncodingContext.bPrepass)
		{
			return; // Nothing gets actually written in the prepass phase
		}

		EncodingContext.Writer->Write(Data, NumBits);
	}

	/** Write a symbol to the bit stream using a Huffman Table */
	FORCEINLINE void WriteSymbol(FHuffmanEncodeTable& table, int32 symbol)
	{
		// Write using the table, will not write anything in the prepass phase
		table.Encode(*EncodingContext.Writer, symbol);
	}

	/** Zero based index of the highest one bit	*/
	FORCEINLINE static int32 HighestSetBit(uint32 Value)
	{
		check(Value != 0); // This obviously doesn't make much sense when nothing is set...						   
		return FPlatformMath::FloorLog2(Value); // This is a differently named thing but essentially does the same
	}

	/** Write which streams are coded in the bit stream */
	void WriteCodedStreamDescription();
	
	/** Any context information to code a frame in a sequence of frames */
	FCodecV1EncodingContext EncodingContext;	
	/** Configuration settings for the encoder, e.g., quality settings */
	FCodecV1EncoderConfig Config;
	
	/** Statistics of the encoding process and individual streams. Primarily used during development, a select number 
	    can be changed to UE-specific counters */
	struct FEncoderStatistics 
	{
		FEncoderStatistics() : NumVertices(0), DurationMs(0.0f), HuffmanTablesNumBytes(0) {}
		
		FStreamEncodingStatistics Indices;
		FStreamEncodingStatistics Vertices;		
		FStreamEncodingStatistics ImportedVertexNumbers;
		FStreamEncodingStatistics Colors;
		FStreamEncodingStatistics TangentX;
		FStreamEncodingStatistics TangentY;
		FStreamEncodingStatistics TexCoords;
		FStreamEncodingStatistics MotionVectors;
		FStreamEncodingStatistics All;
		uint32 NumVertices;
		float DurationMs;
		uint32 HuffmanTablesNumBytes;
	} Statistics;
};

#endif // WITH_EDITOR

class FCodecV1Decoder : public ICodecDecoder
{
public:
	FCodecV1Decoder() {}

	static void InitLUT();

	/**
	* Read a frame's bit stream and decode the frame
	*
	* @param Reader - Reader holding the frame's bit stream
	* @param OutMeshData - Decoded mesh
	*/
	virtual bool DecodeFrameData(FBufferReader &Reader, FGeometryCacheMeshData &OutMeshData);
private:

	/** Decode a buffer of vertex indices from abitstream */
	void DecodeIndexStream(FHuffmanBitStreamReader& Reader, uint32* Stream, uint64 ElementOffset, uint32 ElementCount);
	/** Decode a buffer of vertex positions from a bitstream */
	void DecodePositionStream(FHuffmanBitStreamReader& Reader, FVector3f* VertexStream, uint64 VertexElementOffset, uint32 VertexElementCount);
	/** Decode a buffer of vertex colors from a bitstream */
	void DecodeColorStream(FHuffmanBitStreamReader& Reader, FColor* Stream, uint64 ElementOffset, uint32 ElementCount);
	/** Decode a buffer of vertex normals from a bitstream */
	void DecodeNormalStream(FHuffmanBitStreamReader& Reader, FPackedNormal* Stream, uint64 ElementOffset, uint32 ElementCount, FHuffmanDecodeTable& Table);
	/** Decode a buffer of vertex texture coordinates from a bitstream */
	void DecodeUVStream(FHuffmanBitStreamReader& Reader, FVector2f* Stream, uint64 ElementOffset, uint32 ElementCount);
	/** Decode a buffer of vertex motion vectors from a bitstream */
	void DecodeMotionVectorStream(FHuffmanBitStreamReader& Reader, FVector3f* Stream, uint64 ElementOffset, uint32 ElementCount);
	
	/** Initialize and read Huffman tables from a bitstream */
	void SetupAndReadTables(FHuffmanBitStreamReader& Reader);
	
	/** Read a given number of bytes from a bit stream using a Huffman bitreader */
	FORCEINLINE void ReadBytes(FHuffmanBitStreamReader& Reader, void* Data, uint32 NumBytes);

	/**	Read a 32-bit signed value from a bit stream using a Huffman coder */
	FORCEINLINE int32 ReadInt32(FHuffmanBitStreamReader& Reader, FHuffmanDecodeTable& ValueTable);

	/** Read a symbol from a bit stream using a Huffman Table */
	FORCEINLINE int32 ReadSymbol(FHuffmanBitStreamReader& Reader, FHuffmanDecodeTable& table)
	{
		return table.Decode(Reader);
	}

	/** Read a given number of bits from a bit stream */
	FORCEINLINE int32 ReadBits(FHuffmanBitStreamReader& Reader, uint32 NumBits)
	{
		return Reader.Read(NumBits);
	}

	/** Read a given number of bits from a bit stream without refilling bits */
	FORCEINLINE int32 ReadBitsNoRefill(FHuffmanBitStreamReader& Reader, uint32 NumBits)
	{
		return Reader.ReadNoRefill(NumBits);
	}

	/** Read info on the available streams from a bit stream */
	void ReadCodedStreamDescription(FHuffmanBitStreamReader& Reader, uint32 Version);

	/** Any context information to decode a frame in a sequence of frames, such as any Huffman tables used. */
	FCodecV1DecodingContext DecodingContext;	

	static int32 CachedHighBitsLUT[64];
};

