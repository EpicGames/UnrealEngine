// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CodecV1.h"
#include "Serialization/MemoryWriter.h"
#include "GeometryCacheCodecBase.h"
#include "Misc/FileHelper.h"
#include "GeometryCacheMeshData.h"
#include "MeshBuild.h"

/** 
	Testing functionality to write raw mesh data to file, read it in a testing scenario and run 
	encoder and decoder on the frames. Outputs frame_%i_raw.dump, frame_%i_encoded.dump, frame_%i_decoded.dump.
*/
class CodecV1Test
{
public:
	CodecV1Test(const FString& FrameDirectoryPath)
	{
		for (int32 FrameIndex = 0; FrameIndex < 10; ++FrameIndex) // Test 10 frames
		{
#if WITH_EDITOR
			TestEncoder(FrameIndex, FrameDirectoryPath, FCodecV1EncoderConfig::DefaultConfig());
#endif // WITH_EDITOR
			TestDecoder(FrameIndex, FrameDirectoryPath);
			CompareData(FrameIndex, FrameDirectoryPath);
		}
	}
#if WITH_EDITOR
	static void TestEncoder(int32 FrameIndex, const FString& FrameDirectoryPath, const FCodecV1EncoderConfig& Config)
	{
		// Read raw data from file
		FGeometryCacheMeshData MeshData;
		FString FileNameRaw = FrameDirectoryPath + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_raw.dump"); // frame_%i_raw.dump
		bool bLoadedFile = ReadRawMeshDataFromFile(MeshData, FileNameRaw);

		if (bLoadedFile)
		{
			// Encode object
			FCodecV1Encoder Encoder(Config);
			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes, /*bIsPersistent=*/ true);
			FGeometryCacheCodecEncodeArguments Args(MeshData, 0.0f, false);
			Encoder.EncodeFrameData(Writer, Args);

			// Save encoded bitstream to file
			FString FileNameEncoded = FrameDirectoryPath + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_encoded.dump"); // frame_%i_encoded.dump
			FFileHelper::SaveArrayToFile(Bytes, *FileNameEncoded);
		}
	}
#endif // WITH_EDITOR

	static void TestDecoder(int32 FrameIndex, const FString& FrameDirectoryPath)
	{
		// Read encoded bitstream from file
		FString FileNameEncoded = FrameDirectoryPath + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_encoded.dump"); // frame_%i_encoded.dump
		TArray<uint8> Data;
		bool bLoadedFile = FFileHelper::LoadFileToArray(Data, *FileNameEncoded, FILEREAD_Silent);

		if (bLoadedFile)
		{
			// Decode bitstream
			FCodecV1Decoder Decoder;
			FBufferReader Reader(Data.GetData(), Data.Num(), /*bInFreeOnClose=*/ false, /*bIsPersistent=*/ true);
			FGeometryCacheMeshData MeshDataOut;
			Decoder.DecodeFrameData(Reader, MeshDataOut);

			// Write decoded output to file
			FString FileNameDecoded = FrameDirectoryPath + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_decoded.dump"); // frame_%i_decoded.dump
			WriteRawMeshDataToFile(MeshDataOut, FileNameDecoded);
		}
	}

	static void CompareData(int32 FrameIndex, const FString& FrameDirectoryPath)
	{
		FGeometryCacheMeshData OriginalMeshData;
		FString FileNameRaw = FrameDirectoryPath + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_raw.dump"); // frame_%i_raw.dump
		bool bLoadedFile = ReadRawMeshDataFromFile(OriginalMeshData, FileNameRaw);
		if (bLoadedFile)
		{
			FString FileNameDecoded = FrameDirectoryPath + TEXT("frame_") + FString::FormatAsNumber(FrameIndex) + TEXT("_decoded.dump"); // frame_%i_decoded.dump
			FGeometryCacheMeshData DecodedMeshData;
			bLoadedFile = ReadRawMeshDataFromFile(DecodedMeshData, FileNameDecoded);

			if (bLoadedFile)
			{
				const int32 NumVertices = OriginalMeshData.Positions.Num();
				for (int32 Index = 0; Index < NumVertices; ++Index)
				{
					const FVector3f& PositionA = OriginalMeshData.Positions[Index];
					const FVector3f& PositionB = DecodedMeshData.Positions[Index];

					if (!PointsEqual(PositionA, PositionB, true))
					{
						check(true);
					}
					// The following are already 8 bit so quantized enough we can do exact equal comparisons
					const FPackedNormal& TangentXA = OriginalMeshData.TangentsX[Index];
					const FPackedNormal& TangentXB = DecodedMeshData.TangentsX[Index];

					if (TangentXA != TangentXB)
					{
						check(true);
					}

					const FPackedNormal& TangentZA = OriginalMeshData.TangentsZ[Index];
					const FPackedNormal& TangentZB = DecodedMeshData.TangentsZ[Index];

					if (TangentZA != TangentZB)
					{
						check(true);
					}

					const FColor& ColorA = OriginalMeshData.Colors[Index];
					const FColor& ColorB = DecodedMeshData.Colors[Index];

					if (ColorA != ColorB)
					{
						check(true);
					}

					const FVector2f& UVA = OriginalMeshData.TextureCoordinates[Index];
					const FVector2f& UVB = DecodedMeshData.TextureCoordinates[Index];

					if (!UVsEqual(UVA, UVB))
					{
						check(true);
					}

					// Motion vectors if we have any
					if (OriginalMeshData.Positions.Num() == OriginalMeshData.MotionVectors.Num())
					{
						if (!PointsEqual(OriginalMeshData.MotionVectors[Index], DecodedMeshData.MotionVectors[Index]))
						{
							check(true);
						}
					}
				}
			}
		}
	}

	static void WriteRawMeshDataToFile(const FGeometryCacheMeshData& MeshData, const FString& FileName)
	{
		TArray<uint8> Bytes;
		FMemoryWriter Writer(Bytes, /*bIsPersistent=*/ true);

		Writer << MeshData;

		FFileHelper::SaveArrayToFile(Bytes, *FileName);
	}

	static bool ReadRawMeshDataFromFile(FGeometryCacheMeshData& MeshData, const FString& FileName)
	{
		TArray<uint8> Bytes;
		bool bLoadedFile = FFileHelper::LoadFileToArray(Bytes, *FileName, FILEREAD_Silent);

		if (bLoadedFile)
		{
			FBufferReader Reader(Bytes.GetData(), Bytes.Num(),/*bInFreeOnClose=*/ false, /*bIsPersistent=*/ true);
			Reader << MeshData;
		}

		return bLoadedFile;
	}
};

