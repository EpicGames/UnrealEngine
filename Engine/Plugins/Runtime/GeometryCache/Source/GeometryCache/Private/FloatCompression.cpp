// Copyright Epic Games, Inc. All Rights Reserved.


#include "FloatCompression.h"
#include "HuffmanTable.h"

//#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY_STATIC(LogBlankProgram, Log, All);

//IMPLEMENT_APPLICATION(BlankProgram, "BlankProgram");

/**
	Move data to an 16 byte larger array to make it safe to use with HuffmanBitStreamReader
*/
static TArray<uint8> PadData(const TArray<uint8>& Data)
{
	TArray<uint8> PaddedData;
	PaddedData.SetNumUninitialized(Data.Num() + 16);
	FMemory::Memcpy(&PaddedData[0], Data.GetData(), Data.Num());
	return PaddedData;
}

/**
	Floating point compression/decompression class.
	This is a quick sample class could probably have a nicer interface where the
	stream is not a member but passed in instead.

	QuantizationShift == x means
	Drop the x least significant bits of the mantissa
	(float mantissa is 23 bits so this should be less than 23 ideally)
	Half float has 10 mantissa bits so if you use 13 you're still doing
	about as good as half and probably better as the exponent still has more
	bits than half.
*/
template <uint32 QuantizationShift> class FloatCoder
{
public:

	FloatCoder()
	{
		BitlengthsEncodeTable.Initialize(256);
	}

	/**
		Encode the given array of floats.
	*/
	void Encode(float *Values, int32 NumValues)
	{
		BitlengthsEncodeTable.SetPrepass(true);
		EncodePass(Values, NumValues);
		BitlengthsEncodeTable.SetPrepass(false);
		EncodePass(Values, NumValues);

		uint32 bytes = Stream.GetNumBytes();
		uint32 originalBytes = sizeof(float) * NumValues;
		float rate = static_cast<float>(originalBytes) / static_cast<float>(bytes);
	}

	/**
		Get the encoded data
	*/
	TArray<uint8> GetData() { return Stream.GetBytes(); }

	/**
		Decode previously encoded data
	*/
	void Decode(TArray<uint8> Data, float *Values, int32 NumValues)
	{
		FHuffmanBitStreamReader InputStream(Data.GetData(), Data.Num());

		for (int32 Index = 0; Index < NumValues; Index++)
		{
			float Pred = (Index > 0) ? Values[Index - 1] : 0.0f;
			uint32 PredInt = IntEncode(Pred);
			uint32 PackedK = BitlengthsDecodeTable.Decode(InputStream);
			uint32 Value = 0;
			if (PackedK == Zero)
			{
				Value = PredInt;
			}
			else if (PackedK > Zero)
			{
				uint32 NumBits = PackedK - 1 - Zero;
				uint32 Delta = InputStream.Read(NumBits) + (1 << NumBits);
				Value = PredInt + Delta;
			}
			else /*(packedK < zero)*/
			{
				uint32 NumBits = Zero - PackedK - 1;
				uint32 Delta = InputStream.Read(NumBits) + (1 << NumBits);
				Value = PredInt - Delta;
			}

			Values[Index] = IntDecode(Value);
		}
	}

protected:
	/**
		Encode a float to the internal integer representation.
		If QuantizationShift is not zero this is a lossy operation.
	*/
	static uint32 IntEncode(float Input)
	{
		const uint32 ISNEGATIVEBIT = 0x80000000;
		uint32 IntInput = *((int32 *)&Input);
		uint32 Output = 0;
		if (IntInput&ISNEGATIVEBIT)
		{
			Output = ~IntInput;
		}
		else
		{
			Output = IntInput | ISNEGATIVEBIT;
		}

		// c++ is not clear if this is arithmetic or not but note that we don't
		// care as we shift left again on decoding before doing anything with the data
		Output = Output >> QuantizationShift;
		return Output;
	}

	/**
		Decode a float from the internal integer representation.
	*/
	static float IntDecode(uint32 Input)
	{
		const uint32 ISNEGATIVEBIT = 0x80000000;
		uint32 Output;
		Input = Input << QuantizationShift;
		if (Input&ISNEGATIVEBIT)
		{
			Output = Input & ~ISNEGATIVEBIT;
		}
		else
		{
			Output = ~Input;
		}
		return *((float *)(&Output));
	}

	FHuffmanEncodeTable BitlengthsEncodeTable; //Huffman table for the most significant bit & sign combo.
	FHuffmanDecodeTable BitlengthsDecodeTable;

	FHuffmanBitStreamWriter Stream; // Used to write the encoded data to
	const uint32 Zero = 33; // perfect prediction symbol

	/**
		Zero based index of the highest one bit
	*/
	FORCEINLINE static int32 HighestSetBit(uint32 Value)
	{
		check(Value != 0); // This obviously doesn't make much sense when nothing is set...
		// This is a differently named thing but essentially does the same
		return FPlatformMath::FloorLog2(Value);
	}

	/**
		Do a single encoding pass.
	*/
	void EncodePass(float *Values, int32 NumValues)
	{
		Stream.Clear();

		for (int32 Index = 0; Index < NumValues; Index++)
		{
			float value = Values[Index];
			float pred = (Index > 0) ? Values[Index - 1] : 0.0f;
			uint32 CodedValue = IntEncode(value);
			uint32 PredValue = IntEncode(pred);

			if (PredValue < CodedValue)
			{
				// case 1: under prediction
				uint32 Delta = CodedValue - PredValue; // absolute difference
				uint32 HiBit = HighestSetBit(Delta); // find most significant bit k
				BitlengthsEncodeTable.Encode(Stream, Zero + (HiBit + 1));
				Stream.Write(Delta - (1 << HiBit), HiBit);// code remaining k bits verbatim
			}
			else if (PredValue > CodedValue)
			{
				// case 2: over prediction
				uint32 Delta = PredValue - CodedValue; // absolute difference
				uint32 HiBit = HighestSetBit(Delta); // find most significant bit k
				BitlengthsEncodeTable.Encode(Stream, Zero - (HiBit + 1)); // entropy code k
				Stream.Write(Delta - (1 << HiBit), HiBit); // code remaining k bits verbatim
			}
			else
			{
				// case 3: perfect prediction
				BitlengthsEncodeTable.Encode(Stream, Zero);
			}
		}
		Stream.Close();
	}
};

/************************************************************************/
/*
Bitstream functions testing
*/
/************************************************************************/
void BitstreamTest()
{
	FHuffmanBitStreamWriter TableStream;
	//TableStream.WriteBit(1);
	//TableStream.WriteBit(0);
	uint32 TestValue = 11;
	TableStream.Write(TestValue, 4);
	TableStream.Close();

	// Single read call
	{
		TArray<uint8> PaddedTableData = PadData(TableStream.GetBytes());
		FHuffmanBitStreamReader TableStreamReader(PaddedTableData.GetData(), PaddedTableData.Num());
		uint32 Result = TableStreamReader.Read(4);
		check(Result == 11);
	}

	// Single bit reads
	{
		TArray<uint8> PaddedTableData = PadData(TableStream.GetBytes());
		FHuffmanBitStreamReader TableStreamReader(PaddedTableData.GetData(), PaddedTableData.Num());
		uint32 Result = TableStreamReader.Read();
		check(Result == 1);
		Result = TableStreamReader.Read();
		check(Result == 0);
		Result = TableStreamReader.Read();
		check(Result == 1);
		Result = TableStreamReader.Read();
		check(Result == 1);
	}

	// Sequential bit reads + shifting
	{
		TArray<uint8> PaddedTableData = PadData(TableStream.GetBytes());
		FHuffmanBitStreamReader TableStreamReader(PaddedTableData.GetData(), PaddedTableData.Num());
		uint32 Result = TableStreamReader.Read(3);
		Result = Result << 1;
		Result |= TableStreamReader.Read();
		check(Result == 11);
	}

	// Peeking with zero padding
	{
		TArray<uint8> PaddedTableData = PadData(TableStream.GetBytes());
		FHuffmanBitStreamReader TableStreamReader(PaddedTableData.GetData(), PaddedTableData.Num());
		uint32 Result = TableStreamReader.Peek(8);
		check(Result == 0xB0);
	}

	TableStream.Clear();
	TableStream.Write(TestValue, 4);

	TestValue = 0x9E;
	TableStream.Write(TestValue, 17);

	TestValue = 0xEC;
	TableStream.Write(TestValue, 9);
	TableStream.Close();

	{
		TArray<uint8> PaddedTableData = PadData(TableStream.GetBytes());
		FHuffmanBitStreamReader TableStreamReader(PaddedTableData.GetData(), PaddedTableData.Num());
		uint32 Result = TableStreamReader.Read(4);
		check(Result == 11);
		Result = TableStreamReader.Read(17);
		check(Result == 0x9E);
		Result = TableStreamReader.Read(9);
		check(Result == 0xEC);
	}
}

/************************************************************************/
/* 
Simple coding of an integer buffer
*/
/************************************************************************/
void IntegerTest()
{
	// Create a buffer

	int32 NumValues = 1024 * 1024;
	int32* IntValues = new int32[NumValues];
	int32* OutIntValues = new int32[NumValues];

	for (int32 Index = 0; Index < NumValues; Index++)
	{
		float f = static_cast<float>(Index) / 1024.0f;
		float sf = static_cast<float>(sin(f));
		IntValues[Index] = (uint32)((sf*0.5 + 0.5) * 3999);
	}

	FHuffmanEncodeTable Tab; // 12 bit numbers
	Tab.Initialize(4000);
	FHuffmanBitStreamWriter Writer;
	Tab.SetPrepass(true);
	for (int32 Index = 0; Index < NumValues; Index++)
	{
		Tab.Encode(Writer, IntValues[Index]);
	}
	Tab.SetPrepass(false);
	for (int32 Index = 0; Index < NumValues; Index++)
	{
		Tab.Encode(Writer, IntValues[Index]);
	}
	Writer.Close();

	TArray<uint8> PaddedWriterData = PadData(Writer.GetBytes());
	FHuffmanBitStreamReader Reader(PaddedWriterData.GetData(), PaddedWriterData.Num());

	//Serialize and deserialize the table. This is not needed as the same instance can be used but that way
	//we can test the table serialization also.
	FHuffmanBitStreamWriter TableStream;
	Tab.Serialize(TableStream);
	TableStream.Close();
	TArray<uint8> PaddedTableData = PadData(TableStream.GetBytes());
	FHuffmanBitStreamReader TableStreamReader(PaddedTableData.GetData(), PaddedTableData.Num());
	FHuffmanDecodeTable ReadTable;
	ReadTable.Initialize(TableStreamReader);

	for (int32 Index = 0; Index < NumValues; Index++)
	{
		OutIntValues[Index] = ReadTable.Decode(Reader);
	}

	for (int32 Index = 0; Index < NumValues; Index++)
	{
		check(OutIntValues[Index] == IntValues[Index]);
	}

	delete[] IntValues;
	delete[] OutIntValues;
}


/************************************************************************/
/* Testing of floating point compression                              */
/************************************************************************/
void FloatTest()
{
	// Create a buffer
	int32 NumValues = 1024 * 1024;
	float* Values = new float[NumValues];
	float* OutValues = new float[NumValues];

	for (int32 Index = 0; Index < NumValues; Index++)
	{
		Values[Index] = static_cast<float>(sin(static_cast<float>(Index) / 1024.0f));
	}

	// lossless mode
	FloatCoder<0> coder;

	coder.Encode(Values, NumValues);
	TArray<uint8> PaddedData = PadData(coder.GetData());
	coder.Decode(PaddedData, OutValues, NumValues);

	for (int32 Index = 0; Index < NumValues; Index++)
	{
		check(Values[Index] == OutValues[Index]);
	}

	delete[] Values;
	delete[] OutValues;
}

/*
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	GEngineLoop.PreInit(ArgC, ArgV);
	UE_LOGF(LogBlankProgram, Display, "Hello World2");
	BitstreamTest();
	IntegerTest();
	FloatTest();
	return 0;
}
*/
