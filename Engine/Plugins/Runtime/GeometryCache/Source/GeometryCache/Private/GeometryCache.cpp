// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCache.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/AssetUserData.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/AnimPhysObjectVersion.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/RenderingObjectVersion.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCache)

DEFINE_LOG_CATEGORY(LogGeometryCache);

#define LOCTEXT_NAMESPACE "GeometryCache"

UGeometryCache::UGeometryCache(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	StartFrame = 0;
	EndFrame = 0;
	Hash = 0;
}

void UGeometryCache::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
	Super::PostInitProperties();
}

void UGeometryCache::PreSave(FObjectPreSaveContext SaveContext)
{
	OnPreSave.ExecuteIfBound(this);

	Super::PreSave(SaveContext);
}

void UGeometryCache::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	auto ShowDeprecationNotification = [this]()
	{
		Tracks.Empty();
		Materials.Empty();

		const FText ErrorText = LOCTEXT("GeometryCacheEmptied", "Geometry Cache asset has been emptied as it does not support backwards compatibility");
		FNotificationInfo Info(ErrorText);
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		UE_LOGF(LogGeometryCache, Warning, "(%ls) %ls", *ErrorText.ToString(), *GetName());
	};
		
	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) >= FAnimPhysObjectVersion::GeometryCacheAssetDeprecation)
	{
		Super::Serialize(Ar);

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::GeometryCacheFastDecoder)
		{
			ShowDeprecationNotification();
		}
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (!Ar.IsCooking() || (Ar.CookingTarget() && Ar.CookingTarget()->HasEditorOnlyData()))
		{
			Ar << AssetImportData;
		}
#endif
		Ar << Tracks;

		uint32 NumVertexAnimationTracks;
		uint32 NumTransformAnimationTracks;
		Ar << NumVertexAnimationTracks;
		Ar << NumTransformAnimationTracks;

		if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::GeometryCacheMissingMaterials)
		{
			Ar << Materials;
		}

		ShowDeprecationNotification();
	}
}

FString UGeometryCache::GetDesc()
{
	const int32 NumTracks = Tracks.Num();
	return FString::Printf(TEXT("%d Tracks"), NumTracks);
}

void UGeometryCache::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UGeometryCache::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	// Information on number total of (per type) tracks
	const int32 NumTracks = Tracks.Num();	
	Context.AddTag(FAssetRegistryTag("Total Tracks", FString::FromInt(NumTracks), FAssetRegistryTag::TT_Numerical));

#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif

	Super::GetAssetRegistryTags(Context);
}

void UGeometryCache::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	for(TObjectPtr<UGeometryCacheTrack> Track : Tracks)
	{
		if (Track)
		{
			Track->GetResourceSizeEx(CumulativeResourceSize);
		}
	}
}

void UGeometryCache::BeginDestroy()
{
	Super::BeginDestroy();
	ReleaseResourcesFence.BeginFence();
}

void UGeometryCache::ClearForReimporting()
{
	Materials.Reset();
	MaterialSlotNames.Reset();
	Tracks.Empty();

	// Flush the resource release commands to the rendering thread to ensure that the edit change doesn't occur while a resource is still allocated
	ReleaseResourcesFence.Wait();
}

bool UGeometryCache::IsReadyForFinishDestroy()
{
	return ReleaseResourcesFence.IsFenceComplete();
}

#if WITH_EDITOR
void UGeometryCache::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Flush the resource release commands to the rendering thread to ensure that the edit change doesn't occur while a resource is still allocated
	ReleaseResourcesFence.Wait();
}
#endif

void UGeometryCache::AddTrack(UGeometryCacheTrack* Track)
{
	Tracks.Add(Track);

	uint64 TmpHash = Track->GetHash();
	if (TmpHash)
	{
		Hash = CityHash64WithSeed((char*) &TmpHash, sizeof(uint64), Hash);
	}
}

void UGeometryCache::SetFrameStartEnd(int32 InStartFrame, int32 InEndFrame)
{
	StartFrame = InStartFrame;
	EndFrame = InEndFrame;
}

int32 UGeometryCache::GetStartFrame() const
{
	return StartFrame;
}

int32 UGeometryCache::GetEndFrame() const
{
	return EndFrame;
}

float UGeometryCache::CalculateDuration() const
{
	int32 NumTracks = Tracks.Num();
	float Duration = 0.0f;
	// Create mesh sections for each GeometryCacheTrack
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		const float TrackDuration = Tracks[TrackIndex]->GetDuration();
		Duration = FMath::Max(Duration, TrackDuration);
	}
	return Duration;
}

int32 UGeometryCache::GetFrameAtTime(const float Time) const
{
	const float Duration = CalculateDuration();
	const int32 NumberOfFrames = GetEndFrame() - GetStartFrame() + 1;;
	const float FrameTime = NumberOfFrames > 1 ? Duration / (float)(NumberOfFrames - 1) : 0.0f;
	const int32 NormalizedFrame = FMath::Clamp(FMath::RoundToInt(Time / FrameTime), 0, NumberOfFrames - 1);
	return StartFrame + NormalizedFrame; 

}

void UGeometryCache::GetMeshDataAtTime(float Time, TArray<FGeometryCacheMeshData>& OutMeshData) const
{
	for (UGeometryCacheTrack* Track : Tracks)
	{
		FGeometryCacheMeshData MeshData;
		if (Track->GetMeshDataAtTime(Time, MeshData))
		{
			OutMeshData.Add(MoveTemp(MeshData));
		}
	}
}

FString UGeometryCache::GetHash() const
{
	if (Hash != 0)
	{
		return LexToString(Hash);
	}

	// Hash should be computed during import, but for previously imported GeometryCache without a hash, hash only the first and last frame for performance sake
	uint64 TmpHash = 0;

	auto HashFrame = [this, &TmpHash](float Time)
	{
		TArray<FGeometryCacheMeshData> MeshesData;
		GetMeshDataAtTime(Time, MeshesData);
		for (const FGeometryCacheMeshData& MeshData : MeshesData)
		{
			uint64 MeshHash = MeshData.GetHash();
			if (MeshHash)
			{
				TmpHash = CityHash64WithSeed((char*) &MeshHash, sizeof(uint64), TmpHash);
			}
		}
	};

	HashFrame(0.0f);
	HashFrame(CalculateDuration());

	return LexToString(TmpHash);
}

void UGeometryCache::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UGeometryCache::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void UGeometryCache::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UGeometryCache::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#undef LOCTEXT_NAMESPACE // "GeometryCache"

