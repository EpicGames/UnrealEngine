// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheActor.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheActor)

AGeometryCacheActor::AGeometryCacheActor(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	GeometryCacheComponent = CreateDefaultSubobject<UGeometryCacheComponent>(TEXT("GeometryCacheComponent"));
	RootComponent = GeometryCacheComponent;
}

UGeometryCacheComponent* AGeometryCacheActor::GetGeometryCacheComponent() const
{
	return GeometryCacheComponent;
}

#if WITH_EDITOR
bool AGeometryCacheActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (GeometryCacheComponent && GeometryCacheComponent->GeometryCache)
	{
		Objects.Add(GeometryCacheComponent->GeometryCache);
	}

	return true;
}
#endif

