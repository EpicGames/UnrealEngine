// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheCodecBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheCodecBase)

/*-----------------------------------------------------------------------------
FStreamedGeometryCacheChunk
-----------------------------------------------------------------------------*/

FStreamedGeometryCacheChunk::FStreamedGeometryCacheChunk() : DataSize(0), FirstFrame(0), LastFrame(0)
{
}

void FStreamedGeometryCacheChunk::Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FStreamedGeometryCacheChunk::Serialize"), STAT_StreamedGeometryCacheChunk_Serialize, STATGROUP_LoadTime);

	// We force it not inline that means bulk data won't automatically be loaded when we deserialize
	// later but only when we explicitly take action to load it
	BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
	BulkData.Serialize(Ar, Owner, ChunkIndex,false);
	Ar << DataSize;
	Ar << FirstFrame;
	Ar << LastFrame;
}

void FStreamedGeometryCacheChunk::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	// TODO: Do we include bulk data in here or not!?!
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(DataSize));
}

/*-----------------------------------------------------------------------------
UGeometryCacheCodecBase
-----------------------------------------------------------------------------*/

GEOMETRYCACHE_API UGeometryCacheCodecBase::UGeometryCacheCodecBase(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UObject(ObjectInitializer)
{
	//Nop
}

#if WITH_EDITOR
void UGeometryCacheCodecBase::BeginCoding(TArray<FStreamedGeometryCacheChunk> &AppendChunksTo)
{
	EncoderBaseData.FrameId = -1;
}

void UGeometryCacheCodecBase::CodeFrame(const FGeometryCacheCodecEncodeArguments& Args)
{
	EncoderBaseData.FrameId++;

	if (!Args.bSameTopologyAsPrevious)
	{
		TopologyRanges.Add(EncoderBaseData.FrameId);
	}
}
#endif // WITH_EDITOR

bool UGeometryCacheCodecBase::IsSameTopology(const TArray<int32>& TopologyRanges, int32 FrameA, int32 FrameB)
{
	// No topology changes at all
	if (TopologyRanges.Num() <= 1)
	{
		return true;
	}

	// Binary searching for closest (floored) SampleIndex 
	uint32 MinIndex = 0;
	uint32 MaxIndex = TopologyRanges.Num()-1;

	// In the last open ended range
	if (FrameA >= TopologyRanges[MaxIndex])
	{
		// Should fall in the same range
		return (FrameB >= TopologyRanges[MaxIndex]);
	}
	// The implicit first open ended range
	else if (FrameA < TopologyRanges[MinIndex])
	{
		return (FrameB < TopologyRanges[MinIndex]);
	}

	while (MaxIndex > 0 && MaxIndex > MinIndex)
	{
		int32 Mid = (MinIndex + MaxIndex + 1) / 2;
		if (FrameA < TopologyRanges[Mid])
		{
			MaxIndex = Mid - 1;
		}
		else
		{
			MinIndex = Mid;
		}
	}

	int32 Range =  MinIndex;
	// We can never be in the last range here (We checked that above so the +1 can be unguarded)
	if (FrameB >= TopologyRanges[Range] && FrameB < TopologyRanges[Range + 1])
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool FGeometryCacheCodecRenderStateBase::IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB)
{
	return UGeometryCacheCodecBase::IsSameTopology(TopologyRanges, SampleIndexA, SampleIndexB);
}
