// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheCodecRaw.h"
#include "GeometryCacheMeshData.h"
#include "Serialization/MemoryWriter.h"
#include "GeometryCacheStreamingManager.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheCodecRaw)

UGeometryCacheCodecRaw::UGeometryCacheCodecRaw(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UGeometryCacheCodecBase(ObjectInitializer)
{
	//Nop
}

FGeometryCacheCodecRenderStateBase *UGeometryCacheCodecRaw::CreateRenderState()
{
	return new FGeometryCacheCodecRenderStateRaw(TopologyRanges);
}

bool UGeometryCacheCodecRaw::DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args)
{
	FGeometryCacheCodecRenderStateRaw RenderState(TopologyRanges);
	return RenderState.DecodeSingleFrame(Args);
}

#if WITH_EDITORONLY_DATA
void UGeometryCacheCodecRaw::BeginCoding(TArray<FStreamedGeometryCacheChunk> &AppendChunksTo)
{
	UGeometryCacheCodecBase::BeginCoding(AppendChunksTo);
	EncoderData.AppendChunksTo = &AppendChunksTo;
	EncoderData.CurrentChunkId = -1;
}

void UGeometryCacheCodecRaw::EndCoding()
{
	UGeometryCacheCodecBase::EndCoding();
}

void UGeometryCacheCodecRaw::CodeFrame(const FGeometryCacheCodecEncodeArguments &Args)
{
	UGeometryCacheCodecBase::CodeFrame(Args);

	// Code a frame
	// Serialize to a byte buffer
	TArray<uint8> TempBytes;
	FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
	Ar << Args.MeshData;

	// Finish up any previous chunks
	if (EncoderData.CurrentChunkId >= 0)
	{
		(*EncoderData.AppendChunksTo)[EncoderData.CurrentChunkId].LastFrame = static_cast<float>(GetEncoderFrameNumer() - 1);
	}

	// Create a new chunk for this coded frame
	FStreamedGeometryCacheChunk* NewChunk = new(*EncoderData.AppendChunksTo)FStreamedGeometryCacheChunk();
	EncoderData.CurrentChunkId = EncoderData.AppendChunksTo->Num() - 1;
	
	// The current code always assumes a chunk per frame
	check(EncoderData.CurrentChunkId == GetEncoderFrameNumer());

	NewChunk->DataSize = TempBytes.Num();
	NewChunk->BulkData.Lock(LOCK_READ_WRITE);
	void* NewChunkData = NewChunk->BulkData.Realloc(TempBytes.Num());
	FMemory::Memcpy(NewChunkData, TempBytes.GetData(), TempBytes.Num());
	NewChunk->BulkData.Unlock();

	// Update other fields, note that for the last time start == end this is valid
	// as any time past end will use the last frame no matter what the interval
	NewChunk->FirstFrame = static_cast<float>(GetEncoderFrameNumer());
	NewChunk->LastFrame = static_cast<float>(GetEncoderFrameNumer());
}
#endif

DECLARE_CYCLE_STAT(TEXT("Deserialize Mesh"), STAT_DeserializeMesh, STATGROUP_GeometryCache);

bool FGeometryCacheCodecRenderStateRaw::DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args)
{
	// We have a chunk per frame so chunk id's are just frame id's
	check(Args.Chunks[Args.FrameIdentifier].FirstFrame == static_cast<float>(Args.FrameIdentifier));

	uint32 DataSize = 0;
	const uint8 *data = IGeometryCacheStreamingManager::Get().MapChunk(Args.Track.GetTrack(), Args.FrameIdentifier, &DataSize);
	if (!data)
	{
		return false;
	}

	FBufferReader Ar((void *)data,
		DataSize,
		/*bInFreeOnClose=*/ false, /*bIsPersistent=*/ true
	);

	{
		SCOPE_CYCLE_COUNTER(STAT_DeserializeMesh);
		Ar << Args.OutMeshData;
	}
	IGeometryCacheStreamingManager::Get().UnmapChunk(Args.Track.GetTrack(), Args.FrameIdentifier);
	return true;
}
