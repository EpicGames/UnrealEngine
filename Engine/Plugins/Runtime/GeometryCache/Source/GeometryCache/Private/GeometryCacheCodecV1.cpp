// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheCodecV1.h"
#include "Serialization/MemoryWriter.h"
#include "GeometryCacheStreamingManager.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheModule.h"
#include "CodecV1.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheCodecV1)


UGeometryCacheCodecV1::UGeometryCacheCodecV1(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UGeometryCacheCodecBase(ObjectInitializer)
{
	//Nop
#if WITH_EDITORONLY_DATA
	NextContextId = 1;
#endif

	Decoder = new FCodecV1Decoder();
}

FGeometryCacheCodecRenderStateBase *UGeometryCacheCodecV1::CreateRenderState()
{
	return new FGeometryCacheCodecRenderStateV1(TopologyRanges);
}
#if WITH_EDITORONLY_DATA
void UGeometryCacheCodecV1::InitializeEncoder(float InVertexQuantizationPrecision, int32 InUVQuantizationBitRange)
{
	FCodecV1EncoderConfig Config = FCodecV1EncoderConfig::DefaultConfig();
	Config.VertexQuantizationPrecision = InVertexQuantizationPrecision;
	Config.UVQuantizationBitRange = InUVQuantizationBitRange;
	Encoder = new FCodecV1Encoder(Config);
}
#endif

UGeometryCacheCodecV1::~UGeometryCacheCodecV1()
{
	delete Decoder;
	Decoder = nullptr;

#if WITH_EDITORONLY_DATA
	delete Encoder;
	Encoder = nullptr;
#endif
}

bool UGeometryCacheCodecV1::DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args)
{
	// We have a chunk per frame so chunk id's are just frame id's
	check(Args.Chunks[Args.FrameIdentifier].FirstFrame == static_cast<float>(Args.FrameIdentifier));

	uint32 DataSize = 0;
	const uint8 *data = IGeometryCacheStreamingManager::Get().MapChunk(Args.Track.GetTrack(), Args.FrameIdentifier, &DataSize);

	bool bResult = DecodeBuffer(data, DataSize, Args.OutMeshData);
	IGeometryCacheStreamingManager::Get().UnmapChunk(Args.Track.GetTrack(), Args.FrameIdentifier);
	return bResult;
}

bool UGeometryCacheCodecV1::DecodeBuffer(const uint8* Buffer, uint32 BufferSize, FGeometryCacheMeshData& OutMeshData)
{
	if (!Buffer)
	{
		return false;
	}

	check(Decoder != nullptr);

	FBufferReader Ar((void *) Buffer, 
		BufferSize,
		/*bInFreeOnClose=*/ false, /*bIsPersistent=*/ true
	);

	Decoder->DecodeFrameData(Ar, OutMeshData);
	return true;
}

#if WITH_EDITORONLY_DATA
void UGeometryCacheCodecV1::BeginCoding(TArray<FStreamedGeometryCacheChunk> &AppendChunksTo)
{
	UGeometryCacheCodecBase::BeginCoding(AppendChunksTo);
	EncoderData.AppendChunksTo = &AppendChunksTo;
	EncoderData.CurrentChunkId = -1;
}

void UGeometryCacheCodecV1::EndCoding()
{
	UGeometryCacheCodecBase::EndCoding();
}

void UGeometryCacheCodecV1::CodeFrame(const FGeometryCacheCodecEncodeArguments &Args)
{
	UGeometryCacheCodecBase::CodeFrame(Args);

	// Code a frame
	// Serialize to a byte buffer
	TArray<uint8> TempBytes;
	FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
	check(Encoder != nullptr);
	Encoder->EncodeFrameData(Ar, Args);

	// Finish up any previous chunks
	if (EncoderData.CurrentChunkId >= 0)
	{
		(*EncoderData.AppendChunksTo)[EncoderData.CurrentChunkId].LastFrame = static_cast<float>(GetEncoderFrameNumer()-1);
	}

	// Create a new chunk for this coded frame
	FStreamedGeometryCacheChunk* NewChunk = new(*EncoderData.AppendChunksTo)FStreamedGeometryCacheChunk();
	EncoderData.CurrentChunkId = EncoderData.AppendChunksTo->Num() - 1;

	// We currently have a single chunk per frame
	check(EncoderData.CurrentChunkId == GetEncoderFrameNumer());

	NewChunk->DataSize = TempBytes.Num();
	NewChunk->BulkData.Lock(LOCK_READ_WRITE);
	void* NewChunkData = NewChunk->BulkData.Realloc(TempBytes.Num());
	FMemory::Memcpy(NewChunkData, TempBytes.GetData(), TempBytes.Num());
	NewChunk->BulkData.Unlock();

	// Update other fields, note that for the last time start == end this is valid
	// as any time past end will use the last frame no matter what the interval
	NewChunk->FirstFrame = static_cast<float>(GetEncoderFrameNumer());
	NewChunk->LastFrame = static_cast<float>(GetEncoderFrameNumer());
}

#endif

FGeometryCacheCodecRenderStateV1::FGeometryCacheCodecRenderStateV1(const TArray<int32> &SetTopologyRanges) : FGeometryCacheCodecRenderStateBase(SetTopologyRanges)
{
}

FGeometryCacheCodecRenderStateV1::~FGeometryCacheCodecRenderStateV1()
{	
}

DECLARE_CYCLE_STAT(TEXT("Deserialize Mesh"), STAT_DeserializeMeshV1, STATGROUP_GeometryCache);

bool FGeometryCacheCodecRenderStateV1::DecodeSingleFrame(FGeometryCacheCodecDecodeArguments &Args)
{
	// We have a chunk per frame so chunk id's are just frame id's
	check(Args.Chunks[Args.FrameIdentifier].FirstFrame == static_cast<float>(Args.FrameIdentifier));

	uint32 DataSize = 0;
	const uint8 *data = IGeometryCacheStreamingManager::Get().MapChunk(Args.Track.GetTrack(), Args.FrameIdentifier, &DataSize);
	if (!data)
	{
		return false;
	}

	FBufferReader Ar((void *)data,
		DataSize,
		/*bInFreeOnClose=*/ false, /*bIsPersistent=*/ true
	);

	{
		SCOPE_CYCLE_COUNTER(STAT_DeserializeMeshV1);
		FCodecV1Decoder Decoder;
		Decoder.DecodeFrameData(Ar, Args.OutMeshData);
	}
	IGeometryCacheStreamingManager::Get().UnmapChunk(Args.Track.GetTrack(), Args.FrameIdentifier);
	return true;
}

