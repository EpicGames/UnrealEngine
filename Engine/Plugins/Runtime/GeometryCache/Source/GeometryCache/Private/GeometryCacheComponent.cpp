// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheComponent.h"

#include "GeometryCache.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheModule.h"
#include "GeometryCacheSceneProxy.h"
#include "GeometryCacheStreamingManager.h"
#include "GeometryCacheTrack.h"
#include "Logging/MessageLog.h"
#include "RenderingThread.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheComponent)

#define LOCTEXT_NAMESPACE "GeometryCacheComponent"

DECLARE_CYCLE_STAT(TEXT("Component Tick"), STAT_GeometryCacheComponent_TickComponent, STATGROUP_GeometryCache);

UGeometryCacheComponent::UGeometryCacheComponent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	ElapsedTime = 0.0f;
	bRunning = true;
	bLooping = true;
	bInterpolateFrames = true;
	bExtrapolateFrames = true;
	PlayDirection = 1.0f;
	StartTimeOffset = 0.0f;
	PlaybackSpeed = 1.0f;
	Duration = 0.0f;
	bManualTick = false;
	MotionVectorScale = 1.f;
	bOverrideWireframeColor = false;
	WireframeOverrideColor = FLinearColor::Green;
}

void UGeometryCacheComponent::BeginDestroy()
{
	Super::BeginDestroy();
	ReleaseResources();
	// Make sure to remove the component from the streaming manager
	// There are some flows with Sequencer and PIE where it is removed in OnUnregister but
	// re-added through SetGeometryCache
	IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(this);
}

void UGeometryCacheComponent::FinishDestroy()
{
	Super::FinishDestroy();
}

void UGeometryCacheComponent::PostLoad()
{
	Super::PostLoad();
	
	// Precache PSOs for the used materials
	PrecachePSOs();
}

void UGeometryCacheComponent::OnRegister()
{
	ClearTrackData();
	SetupTrackData();
	IGeometryCacheStreamingManager::Get().AddStreamingComponent(this);
	Super::OnRegister();

	// Precache PSOs for the used materials
	PrecachePSOs();
}

void UGeometryCacheComponent::ClearTrackData()
{
	NumTracks = 0;
	TrackSections.Empty();
}

void UGeometryCacheComponent::JumpAnimationToTime(float Time, bool bInIsRunning, bool bInBackwards, bool bInIsLooping)
{
	ElapsedTime = Time;

	// Game thread update:
	// This mainly just updates the matrix and bounding boxes. All render state (meshes) is done on the render thread
	bool bUpdatedBoundsOrMatrix = false;
	for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
	{
		bUpdatedBoundsOrMatrix |= UpdateTrackSection(TrackIndex, Time);
	}

	if (bUpdatedBoundsOrMatrix)
	{
		UpdateLocalBounds();
		// Mark the transform as dirty, so the bounds are updated and sent to the render thread
		MarkRenderTransformDirty();
	}

	// The actual current playback speed. The PlaybackSpeed variable contains the speed it would
	// play back at if it were running regardless of if we're running or not. The renderer
	// needs the actual playback speed if not a paused animation with explicit motion vectors
	// would just keep on blurring as if it were moving even when paused.
	float ActualPlaybackSpeed = (bInIsRunning) ? PlaybackSpeed : 0.0f;

	// Schedule an update on the render thread
	if (FGeometryCacheSceneProxy* CastedProxy = static_cast<FGeometryCacheSceneProxy*>(SceneProxy))
	{
		FGeometryCacheSceneProxy* InSceneProxy = CastedProxy;
		FGeometryCacheSceneProxy::FAnimationUpdateParameters UpdatedValues =
		{
			Time,
			bInIsLooping,
			bInBackwards,
			ActualPlaybackSpeed,
			GetMotionVectorScale(),
			IsInterpolatingFrames()
		};
		ENQUEUE_RENDER_COMMAND(FGeometryCacheUpdateAnimation)(
			[InSceneProxy, UpdatedValues](FRHICommandList& RHICmdList)
			{
				InSceneProxy->UpdateAnimation(RHICmdList, UpdatedValues);
			});
	}
}

void UGeometryCacheComponent::StepAnimationFrame(bool bInBackwards)
{
	const FFrameRate TargetFramerate = FFrameRate(FMath::RoundToInt32((float)GetNumberOfFrames() / GetDuration()), 1);
	const float DeltaTime = static_cast<float>(TargetFramerate.AsInterval());
	const float UnclampedTime = ElapsedTime + (DeltaTime * PlayDirection * GetPlaybackSpeed());
	const float NewTickedTime = IsLooping()
		? GeometyCacheHelpers::WrapAnimationTime(UnclampedTime, Duration)
		: FMath::Clamp(UnclampedTime, 0.0f, Duration);

	JumpAnimationToTime(NewTickedTime, IsPlaying(), bInBackwards, IsLooping());
}

void UGeometryCacheComponent::SetupTrackData()
{
	if (GeometryCache != nullptr)
	{
		// Refresh NumTracks and clear Index Arrays
		NumTracks = GeometryCache->Tracks.Num();

		Duration = 0.0f;
		// Create mesh sections for each GeometryCacheTrack
		for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
		{
			// First time so create rather than update the mesh sections
			CreateTrackSection(TrackIndex);

			const float TrackDuration = GeometryCache->Tracks[TrackIndex]->GetDuration();
			Duration = FMath::Max(Duration, TrackDuration);
		}
	}
	UpdateLocalBounds();
}

void UGeometryCacheComponent::OnUnregister()
{
	IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(this);
	Super::OnUnregister();
	ClearTrackData();
}

void UGeometryCacheComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	SCOPE_CYCLE_COUNTER(STAT_GeometryCacheComponent_TickComponent);
	if (GeometryCache && bRunning && !bManualTick)
	{
		// Increase total elapsed time since BeginPlay according to PlayDirection and speed
		ElapsedTime += (DeltaTime * PlayDirection * GetPlaybackSpeed());

		if (bLooping && (ElapsedTime < 0.0f))
		{
			ElapsedTime += Duration;
		}
		else if (!bLooping)
		{
			if(ElapsedTime < 0.0f)
			{
				ElapsedTime = 0.0f;
				bRunning = false;
			}
			else if (ElapsedTime > Duration)
			{
				ElapsedTime = Duration;
				bRunning = false;
			}
		}
		
		// Game thread update:
		// This mainly just updates the matrix and bounding boxes. All render state (meshes) is done on the render thread
		bool bUpdatedBoundsOrMatrix = false;
		for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
		{
			bUpdatedBoundsOrMatrix |= UpdateTrackSection(TrackIndex, GetAnimationTime());
		}

		if (bUpdatedBoundsOrMatrix)
		{
			UpdateLocalBounds();
		}
		// Mark the transform as dirty, so the bounds are updated and sent to the render thread
		MarkRenderTransformDirty();

		// The actual current playback speed. The PlaybackSpeed variable contains the speed it would
		// play back at if it were running regardless of if we're running or not. The renderer
		// needs the actual playback speed if not a paused animation with explicit motion vectors
		// would just keep on blurring as if it were moving even when paused.
		float ActualPlaybackSpeed = (bRunning) ? PlaybackSpeed : 0.0f;

		// Schedule an update on the render thread
		if (FGeometryCacheSceneProxy* CastedProxy = static_cast<FGeometryCacheSceneProxy*>(SceneProxy))
		{
			FGeometryCacheSceneProxy* InSceneProxy = CastedProxy;
			FGeometryCacheSceneProxy::FAnimationUpdateParameters UpdatedValues =
			{
				GetAnimationTime(),
				IsLooping(),
				IsPlayingReversed(),
				ActualPlaybackSpeed,
				GetMotionVectorScale(),
				IsInterpolatingFrames()
			};
			ENQUEUE_RENDER_COMMAND(FGeometryCacheUpdateAnimation)(
				[InSceneProxy, UpdatedValues](FRHICommandList& RHICmdList)
				{
					InSceneProxy->UpdateAnimation(RHICmdList, UpdatedValues);
				});
		}
	}
}

void UGeometryCacheComponent::SetManualTick(bool bInManualTick)
{
	bManualTick = bInManualTick;
}
bool UGeometryCacheComponent::GetManualTick() const
{
	return bManualTick;
}

void UGeometryCacheComponent::ResetAnimationTime()
{
	ElapsedTime = 0.0f;
}

void UGeometryCacheComponent::TickAtThisTime(const float Time, bool bInIsRunning, bool bInBackwards, bool bInIsLooping)
{
	if (bManualTick && GeometryCache && bRunning)
	{
		JumpAnimationToTime(Time, bInIsRunning, bInBackwards, bInIsLooping);
	}
}

FBoxSphereBounds UGeometryCacheComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return LocalBounds.TransformBy(LocalToWorld);
}

/**
	Update the local bounds of this component based on the bounds of all the tracks in this component.
	This is used to accelerate CalcBounds above.
*/
void UGeometryCacheComponent::UpdateLocalBounds()
{
	FBox LocalBox(ForceInit);

	for (const FTrackRenderData& Section : TrackSections)
	{
		// Use World matrix per section for correct bounding box
		LocalBox += (Section.BoundingBox.TransformBy(Section.Matrix));
	}

	LocalBounds = LocalBox.IsValid ? FBoxSphereBounds(LocalBox) : FBoxSphereBounds(FVector(0, 0, 0), FVector(0, 0, 0), 0); // fall back to reset box sphere bounds

	// This calls CalcBounds above and finally stores the world bounds in the "Bounds" member variable
	UpdateBounds();
}

void UGeometryCacheComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{	
	FGeometryCacheVertexVertexFactory::FDataType Data;
	FGeomCacheVertexFactory::InitDummyComponentVF(Data);

	FVertexDeclarationElementList Elements;
	FGeometryCacheVertexVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, Data, Elements);

	const bool bReverseCulling = GetRenderMatrix().Determinant() < 0;

	for (UMaterialInterface* Material : GetMaterials())
	{
		if (Material == nullptr || !Material->CheckMaterialUsage_Concurrent(MATUSAGE_GeometryCache))
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
		ComponentParams.Priority = EPSOPrecachePriority::Medium;
		ComponentParams.MaterialInterface = Material;
		ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FGeometryCacheVertexVertexFactory::StaticType, Elements));
		ComponentParams.PSOPrecacheParams = BasePrecachePSOParams;
		ComponentParams.PSOPrecacheParams.bReverseCulling |= bReverseCulling;
	}

	UMaterialInterface* OverlayMaterialInterface = GetOverlayMaterial();
	if (OverlayMaterialInterface && OverlayMaterialInterface->CheckMaterialUsage_Concurrent(MATUSAGE_GeometryCache))
	{
		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams.AddDefaulted_GetRef();
		ComponentParams.Priority = EPSOPrecachePriority::Medium;
		ComponentParams.MaterialInterface = OverlayMaterialInterface;
		ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FGeometryCacheVertexVertexFactory::StaticType, Elements));
		ComponentParams.PSOPrecacheParams = BasePrecachePSOParams;
		ComponentParams.PSOPrecacheParams.bReverseCulling |= bReverseCulling;
	}
}

FPrimitiveSceneProxy* UGeometryCacheComponent::CreateSceneProxy()
{
	IGeometryCacheStreamingManager::Get().PrefetchData(this);
	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogGeometryCache, Verbose, "Skipping CreateSceneProxy for GeometryCacheComponent %ls (PSOs are still compiling)", *GetFullName());
		return nullptr;
	}
	return new FGeometryCacheSceneProxy(this);
}

#if WITH_EDITOR
void UGeometryCacheComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InvalidateTrackSampleIndices();
	MarkRenderStateDirty();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

int32 UGeometryCacheComponent::GetNumMaterials() const
{
	return GeometryCache ? GeometryCache->Materials.Num() : 0;
}

UMaterialInterface* UGeometryCacheComponent::GetMaterial(int32 MaterialIndex) const
{
	// If we have a base materials array, use that
	if (OverrideMaterials.IsValidIndex(MaterialIndex) && OverrideMaterials[MaterialIndex])
	{
		return OverrideMaterials[MaterialIndex];
	}
	// Otherwise get it from the geometry cache
	else
	{
		return GeometryCache ? (GeometryCache->Materials.IsValidIndex(MaterialIndex) ? GeometryCache->Materials[MaterialIndex] : nullptr) : nullptr;
	}
}

int32 UGeometryCacheComponent::GetMaterialIndex(FName MaterialSlotName) const
{
	return GeometryCache ? GeometryCache->MaterialSlotNames.IndexOfByKey(MaterialSlotName) : INDEX_NONE;
}

TArray<FName> UGeometryCacheComponent::GetMaterialSlotNames() const
{
	return GeometryCache ? GeometryCache->MaterialSlotNames : TArray<FName>();
}

bool UGeometryCacheComponent::IsMaterialSlotNameValid(FName MaterialSlotName) const
{
	return GetMaterialIndex(MaterialSlotName) != INDEX_NONE;
}

void UGeometryCacheComponent::CreateTrackSection(int32 TrackIndex)
{
	// Ensure sections array is long enough
	if (TrackSections.Num() <= TrackIndex)
	{
		TrackSections.SetNum(TrackIndex + 1, EAllowShrinking::No);
	}

	UpdateTrackSection(TrackIndex, GetAnimationTime());
	MarkRenderStateDirty(); // Recreate scene proxy
}

bool UGeometryCacheComponent::UpdateTrackSection(int32 TrackIndex, float Time)
{
	checkf(TrackIndex < TrackSections.Num() && GeometryCache != nullptr && TrackIndex < GeometryCache->Tracks.Num(), TEXT("Invalid SectionIndex"));

	UGeometryCacheTrack* Track = GeometryCache->Tracks[TrackIndex];
	FTrackRenderData& UpdateSection = TrackSections[TrackIndex];

	Track->UpdateTime(Time, bLooping);

	FMatrix Matrix;
	FBox TrackBounds;
	const bool bUpdateMatrix = Track->UpdateMatrixData(Time, bLooping, UpdateSection.MatrixSampleIndex, Matrix);
	const bool bUpdateBounds = Track->UpdateBoundsData(Time, bLooping, (PlayDirection < 0.0f) ? true : false, UpdateSection.BoundsSampleIndex, TrackBounds);

	if (bUpdateBounds)
	{
		UpdateSection.BoundingBox = TrackBounds;
	}

	if (bUpdateMatrix)
	{
		UpdateSection.Matrix = Matrix;
	}

	// Update sections according what is required
	if (bUpdateMatrix || bUpdateBounds)
	{
		return true;
	}

	return false;
}

void UGeometryCacheComponent::OnObjectReimported(UGeometryCache* ImportedGeometryCache)
{
	if (GeometryCache == ImportedGeometryCache)
	{
		ReleaseResources();
		DetachFence.Wait();

		GeometryCache = ImportedGeometryCache;
		MarkRenderStateDirty();
	}
}

void UGeometryCacheComponent::Play()
{
	bRunning = true;
	PlayDirection = 1.0f;
	IGeometryCacheStreamingManager::Get().PrefetchData(this);
}

void UGeometryCacheComponent::PlayFromStart()
{
	ElapsedTime = 0.0f;
	bRunning = true;
	PlayDirection = 1.0f;
	IGeometryCacheStreamingManager::Get().PrefetchData(this);
}

void UGeometryCacheComponent::Pause()
{
	bRunning = !bRunning;
}

void UGeometryCacheComponent::Stop()
{
	bRunning = false;
}

bool UGeometryCacheComponent::IsPlaying() const
{
	return bRunning;
}

bool UGeometryCacheComponent::IsLooping() const
{
	return bLooping;
}

void UGeometryCacheComponent::SetLooping(const bool bNewLooping)
{
	bLooping = bNewLooping;
}

bool UGeometryCacheComponent::IsInterpolatingFrames() const
{
	return bInterpolateFrames;
}

void UGeometryCacheComponent::SetInterpolateFrames(const bool bNewInterpolating)
{
	bInterpolateFrames = bNewInterpolating;
}

bool UGeometryCacheComponent::IsExtrapolatingFrames() const
{
	return bExtrapolateFrames;
}

void UGeometryCacheComponent::SetExtrapolateFrames(const bool bNewExtrapolating)
{
	bExtrapolateFrames = bNewExtrapolating;
}

bool UGeometryCacheComponent::IsPlayingReversed() const
{
	return FMath::IsNearlyEqual(PlayDirection, -1.0f);
}

float UGeometryCacheComponent::GetPlaybackSpeed() const
{
	return FMath::Clamp(PlaybackSpeed, 0.0f, 512.0f);
}

void UGeometryCacheComponent::SetPlaybackSpeed(const float NewPlaybackSpeed)
{
	// Currently only positive play back speeds are supported
	PlaybackSpeed = FMath::Clamp(NewPlaybackSpeed, 0.0f, 512.0f);
}

float UGeometryCacheComponent::GetMotionVectorScale() const
{
	return FMath::Clamp(MotionVectorScale, 0.0f, 100.0f);
}

void UGeometryCacheComponent::SetMotionVectorScale(const float NewMotionVectorScale)
{
	MotionVectorScale = FMath::Clamp(NewMotionVectorScale, 0.0f, 100.0f);
}

bool UGeometryCacheComponent::SetGeometryCache(UGeometryCache* NewGeomCache)
{
	// Do nothing if we are already using the supplied geometry cache
	if (NewGeomCache == GeometryCache)
	{
		return false;
	}

	// Don't allow changing static meshes if "static" and registered
	AActor* Owner = GetOwner();
	if (!AreDynamicDataChangesAllowed() && Owner != nullptr)
	{
		FMessageLog("PIE").Warning(FText::Format(LOCTEXT("SetGeometryCache", "Calling SetGeometryCache on '{0}' but Mobility is Static."),
			FText::FromString(GetPathName())));
		return false;
	}

	IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(this);

	ReleaseResources();
	DetachFence.Wait();

	GeometryCache = NewGeomCache;

	ClearTrackData();
	SetupTrackData();

	// This will cause us to prefetch the new data which is needed by the render state creation
	IGeometryCacheStreamingManager::Get().AddStreamingComponent(this);

	PrecachePSOs();

	// Need to send this to render thread at some point
	MarkRenderStateDirty();

	// Update physics representation right away
	RecreatePhysicsState();

	// Update this component streaming data.
	IStreamingManager::Get().NotifyPrimitiveUpdated(this);

	// Since we have new tracks, we need to update bounds
	UpdateBounds();
	return true;
}

UGeometryCache* UGeometryCacheComponent::GetGeometryCache() const
{
	return GeometryCache;
}

float UGeometryCacheComponent::GetStartTimeOffset() const
{
	return StartTimeOffset;
}

void UGeometryCacheComponent::SetStartTimeOffset(const float NewStartTimeOffset)
{
	if (!FMath::IsNearlyEqual(StartTimeOffset, NewStartTimeOffset))
	{
		StartTimeOffset = NewStartTimeOffset;
		MarkRenderStateDirty();
	}
}

float UGeometryCacheComponent::GetAnimationTime() const
{
	const float ClampedStartTimeOffset = FMath::Clamp(StartTimeOffset, -14400.0f, 14400.0f);
	return ElapsedTime + ClampedStartTimeOffset;
}

float UGeometryCacheComponent::GetElapsedTime() const
{
	return ElapsedTime;
}

float UGeometryCacheComponent::GetPlaybackDirection() const
{
	return PlayDirection;
}

float UGeometryCacheComponent::GetDuration() const
{
	return Duration;
}

void UGeometryCacheComponent::PlayReversedFromEnd()
{
	ElapsedTime = Duration;
	PlayDirection = -1.0f;
	bRunning = true;
	IGeometryCacheStreamingManager::Get().PrefetchData(this);
}

void UGeometryCacheComponent::PlayReversed()
{
	PlayDirection = -1.0f;
	bRunning = true;
	IGeometryCacheStreamingManager::Get().PrefetchData(this);
}

void UGeometryCacheComponent::InvalidateTrackSampleIndices()
{
	for (FTrackRenderData& Track : TrackSections)
	{
		Track.MatrixSampleIndex = -1;
		Track.BoundsSampleIndex = -1;
	}
}

void UGeometryCacheComponent::ReleaseResources()
{
	GeometryCache = nullptr;
	NumTracks = 0;
	TrackSections.Empty();
	DetachFence.BeginFence();
}

int32 UGeometryCacheComponent::GetFrameAtTime(const float Time) const
{
	const float FrameTime = GetNumberOfFrames() > 1 ? Duration / (float)(GetNumberOfFrames() - 1) : 0.0f;
	const int32 NormalizedFrame = FMath::Clamp(FMath::RoundToInt(Time / FrameTime), 0, GetNumberOfFrames() - 1);
	const int32 StartFrame = GeometryCache != nullptr ? GeometryCache->GetStartFrame() : 0;
	return StartFrame + NormalizedFrame; //
}

float UGeometryCacheComponent::GetTimeAtFrame(const int32 Frame) const
{
	const float FrameTime = GetNumberOfFrames() > 1 ? Duration / (float)(GetNumberOfFrames() - 1) : 0.0f;
	const int32 StartFrame = GeometryCache != nullptr ? GeometryCache->GetStartFrame() : 0;

	return FMath::Clamp(FrameTime * static_cast<float>(Frame - StartFrame), 0.0f, Duration);
}

void UGeometryCacheComponent::SetCurrentTime(const float Time)
{
	if (IsPlaying())
	{
		Stop();
	}

	JumpAnimationToTime(FMath::Clamp(Time, 0.0f, Duration), IsPlaying(), IsPlayingReversed(), IsLooping());
}

int32 UGeometryCacheComponent::GetNumberOfFrames() const
{
	if (GeometryCache)
	{
		return GeometryCache->GetEndFrame() - GeometryCache->GetStartFrame() + 1;
	}
	return 0;
}

int32 UGeometryCacheComponent::GetNumberOfTracks() const
{
	if (GeometryCache)
	{
		return GeometryCache->Tracks.Num();
	}

	return 0;
}

void UGeometryCacheComponent::SetOverrideWireframeColor(bool bOverride)
{
	bOverrideWireframeColor = bOverride;
}

bool UGeometryCacheComponent::GetOverrideWireframeColor() const
{
	return bOverrideWireframeColor;
}

void UGeometryCacheComponent::SetWireframeOverrideColor(FLinearColor Color)
{
	WireframeOverrideColor = Color;
}

FLinearColor UGeometryCacheComponent::GetWireframeOverrideColor() const
{
	return WireframeOverrideColor;
}

#if WITH_EDITOR
void UGeometryCacheComponent::PreEditUndo()
{
	InvalidateTrackSampleIndices();
	MarkRenderStateDirty();
}

void UGeometryCacheComponent::PostEditUndo()
{
	InvalidateTrackSampleIndices();
	MarkRenderStateDirty();
}

void UGeometryCacheComponent::StepForward()
{
	if (IsPlaying())
	{
		Stop();
	}
	PlayDirection = 1.0f;
	constexpr bool bInBackwards = false;
	StepAnimationFrame(bInBackwards);
}

void UGeometryCacheComponent::ForwardEnd()
{
	if (IsPlaying())
	{
		Stop();
	}
	JumpAnimationToTime(Duration, IsPlaying(), IsPlayingReversed(), IsLooping());
}

void UGeometryCacheComponent::StepBackward()
{
	if (IsPlaying())
	{
		Stop();
	}
	PlayDirection = -1.0f;
	constexpr bool bInBackwards = true;
	StepAnimationFrame(bInBackwards);
}

void UGeometryCacheComponent::BackwardEnd()
{
	if (IsPlaying())
	{
		Stop();
	}
	JumpAnimationToTime(0.0f, IsPlaying(), IsPlayingReversed(), IsLooping());
}

void UGeometryCacheComponent::ToggleLooping()
{	
	SetLooping(!bLooping);
	if (!bLooping)
	{
		JumpAnimationToTime(GeometyCacheHelpers::WrapAnimationTime(ElapsedTime, Duration), IsPlaying(), IsPlayingReversed(), IsLooping());
	}	
}

TArray<FString> UGeometryCacheComponent::GetTrackNames() const
{
	TArray<FString> Names;
	for (const TObjectPtr<UGeometryCacheTrack>& Track : GeometryCache->Tracks)
	{
		Names.Add(Track->GetName());
	}
	return Names;
}
#endif

#undef LOCTEXT_NAMESPACE

