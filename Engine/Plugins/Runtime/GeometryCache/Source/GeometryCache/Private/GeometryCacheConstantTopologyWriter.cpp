// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheConstantTopologyWriter.h"

#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCacheWriter, Log, All);

namespace UE::GeometryCacheHelpers
{
#if WITH_EDITOR
	namespace Private
	{
		FBox3f GetBoundingBox(const TArray<FVector3f>& Positions)
		{
			FBox3f BoundingBox;
			for (const FVector3f& Position : Positions)
			{
				BoundingBox += Position;
			}
			return BoundingBox;
		}

		TArray<FVector3f> ComputeNormals(TConstArrayView<uint32> Indices, TConstArrayView<FVector3f> Positions)
		{
			TArray<FVector3f> FaceNormals;
			const int32 NumFaces = Indices.Num() / 3;
			FaceNormals.SetNumUninitialized(NumFaces);
			for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
			{
				const FVector3f& A = Positions[Indices[FaceIndex * 3 + 0]];
				const FVector3f& B = Positions[Indices[FaceIndex * 3 + 1]];
				const FVector3f& C = Positions[Indices[FaceIndex * 3 + 2]];
				FaceNormals[FaceIndex] = (C - A).Cross(B - A);
			}
			TArray<FVector3f> VertexNormals;
			const int32 NumVertices = Positions.Num();
			VertexNormals.SetNumZeroed(NumVertices);
			for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
			{
				const FVector3f& FaceNormal = FaceNormals[FaceIndex];
				VertexNormals[Indices[FaceIndex * 3 + 0]] += FaceNormal;
				VertexNormals[Indices[FaceIndex * 3 + 1]] += FaceNormal;
				VertexNormals[Indices[FaceIndex * 3 + 2]] += FaceNormal;
			}
			const FVector3f DefaultVector(0.0f, 0.0f, 1.0f);
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				VertexNormals[VertexIndex] = VertexNormals[VertexIndex].GetSafeNormal(UE_SMALL_NUMBER, DefaultVector);
			}
			return VertexNormals;
		}

		TArray<FVector3f> ComputeTangentsX(TConstArrayView<uint32> Indices, TConstArrayView<FVector3f> Positions, TConstArrayView<FVector3f> Normals, TConstArrayView<FVector2f> UVs)
		{
			TArray<FVector3f> Tangents;
			const int32 NumVertices = Positions.Num();
			Tangents.SetNumZeroed(NumVertices);
			const int32 NumFaces = Indices.Num() / 3;
			for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
			{
				const int32 Index0 = Indices[FaceIndex * 3];
				const int32 Index1 = Indices[FaceIndex * 3 + 1];
				const int32 Index2 = Indices[FaceIndex * 3 + 2];
				const FVector3f& P0 = Positions[Index0];
				const FVector3f& P1 = Positions[Index1];
				const FVector3f& P2 = Positions[Index2];
				const FVector2f& UV0 = UVs[Index0];
				const FVector2f& UV1 = UVs[Index1];
				const FVector2f& UV2 = UVs[Index2];
		
				const FVector3f Edge1 = P1 - P0;
				const FVector3f Edge2 = P2 - P0;
				const FVector2f UVEdge1 = UV1 - UV0;
				const FVector2f UVEdge2 = UV2 - UV0;
				const float Det = UVEdge1.Y * UVEdge2.X - UVEdge1.X * UVEdge2.Y;
				if (FMath::Abs(Det) > UE_SMALL_NUMBER)
				{
					FVector3f TangentX = (-Edge1 * UVEdge2.Y + Edge2 * UVEdge1.Y) / Det;
					TangentX.Normalize();
					TangentX *= Edge1.Cross(Edge2).Size();
					Tangents[Index0] += TangentX;
					Tangents[Index1] += TangentX;
					Tangents[Index2] += TangentX;
				}
			}
			for (int32 Index = 0; Index < NumVertices; Index++)
			{
				FVector3f& TangentX = Tangents[Index];
				const FVector3f& Normal = Normals[Index];
				TangentX -= Normal * Normal.Dot(TangentX);
				TangentX = TangentX.GetSafeNormal(UE_SMALL_NUMBER, FVector3f(1.0f, 0.0f, 0.0f));
			}
			return Tangents;
		}

		TArray<FPackedNormal> GetPackedNormals(TConstArrayView<FVector3f> Normals)
		{
			TArray<FPackedNormal> PackedNormals;
			PackedNormals.SetNumUninitialized(Normals.Num());
			for (int32 Index = 0; Index < Normals.Num(); ++Index)
			{
				PackedNormals[Index] = Normals[Index];
			}
			return PackedNormals;
		}

		int32 GetNumVertices(const FSkeletalMeshLODRenderData& LODData)
		{
			int32 NumVertices = 0;
			for(const FSkelMeshRenderSection& Section : LODData.RenderSections)
			{
				NumVertices += Section.NumVertices;
			}
			return NumVertices;
		}

		TArray<FVector2f> GetUV0s(const FSkeletalMeshLODRenderData& LODData)
		{
			TArray<FVector2f> UV0s;
			const FStaticMeshVertexBuffer& StaticMeshVertexBuffer = LODData.StaticVertexBuffers.StaticMeshVertexBuffer;
			const int32 NumVertices = StaticMeshVertexBuffer.GetNumVertices();
			UV0s.SetNumZeroed(NumVertices);
			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				UV0s[Index] = StaticMeshVertexBuffer.GetVertexUV(Index, 0);
			}
			return UV0s;
		}
		
		TArray<FColor> GetColors(const FSkeletalMeshLODRenderData& LODData, int32 NumVertices)
		{
			TArray<FColor> Colors;
			Colors.SetNum(NumVertices);
			const FColorVertexBuffer& ColorVertexBuffer = LODData.StaticVertexBuffers.ColorVertexBuffer;
			if (ColorVertexBuffer.GetNumVertices() == NumVertices)
			{
				for (int32 Index = 0; Index < NumVertices; ++Index)
				{
					Colors[Index] = ColorVertexBuffer.VertexColor(Index);
				}
			}
			else
			{
				for (int32 Index = 0; Index < NumVertices; ++Index)
				{
					Colors[Index] = FColor::White;
				}
			}
			return Colors;
		}

		TArray<TObjectPtr<UMaterialInterface>> GetMaterialInterfaces(const USkinnedAsset& Asset)
		{
			const TArray<FSkeletalMaterial>& Materials = Asset.GetMaterials();
			TArray<TObjectPtr<UMaterialInterface>> Interfaces;
			Interfaces.SetNum(Materials.Num());
			for (int32 Index = 0; Index < Materials.Num(); ++Index)
			{
				Interfaces[Index] = Materials[Index].MaterialInterface;
			}
			return Interfaces;
		}

		TArray<FGeometryCacheMeshBatchInfo> GetBatchesInfo(const FSkeletalMeshLODRenderData& LODData, int32 MaterialOffset)
		{
			TArray<FGeometryCacheMeshBatchInfo> BatchesInfo;
			BatchesInfo.Reserve(LODData.RenderSections.Num());
			for (const FSkelMeshRenderSection& Section : LODData.RenderSections)
			{
				FGeometryCacheMeshBatchInfo BatchInfo;
				BatchInfo.StartIndex = Section.BaseIndex;
				BatchInfo.NumTriangles = Section.NumTriangles;
				BatchInfo.MaterialIndex = Section.MaterialIndex + MaterialOffset;
				BatchesInfo.Add(MoveTemp(BatchInfo));
			}
			return BatchesInfo;
		}

		TArray<uint32> GetImportedVertexNumbers(const USkinnedAsset& Asset, int32 LODIndex)
		{
			const FSkeletalMeshModel* const ImportedModel = Asset.GetImportedModel();
			if (!ImportedModel || !ImportedModel->LODModels.IsValidIndex(LODIndex))
			{
				return TArray<uint32>();
			}
			const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
			const TArray<int32>& Map = LODModel.MeshToImportVertexMap;
			return TArray<uint32>(reinterpret_cast<const uint32*>(Map.GetData()), Map.Num());
		}

		template<typename T>
		TArray<T> GetSubArray(const TArray<T>& Array, int32 Start, int32 End)
		{
			if (Start >= End || Start >= Array.Num())
			{
				return TArray<T>();
			}
			Start = FMath::Clamp(Start, 0, Array.Num() - 1);
			End = FMath::Clamp(End, 0, Array.Num());
			return TArray<T>(Array.GetData() + Start, End - Start);
		}
	}

	FGeometryCacheConstantTopologyWriter::FGeometryCacheConstantTopologyWriter(UGeometryCache& OutCache)
		: FGeometryCacheConstantTopologyWriter(OutCache, FConfig())
	{
	}

	FGeometryCacheConstantTopologyWriter::FGeometryCacheConstantTopologyWriter(UGeometryCache& OutCache, const FConfig& InConfig)
		: Cache(&OutCache)
		, Config(InConfig)
	{
		Cache->ClearForReimporting();
		Cache->Materials.Empty();
	}

	FGeometryCacheConstantTopologyWriter::~FGeometryCacheConstantTopologyWriter() = default;
	
	using FTrackWriter = FGeometryCacheConstantTopologyWriter::FTrackWriter;
	FTrackWriter& FGeometryCacheConstantTopologyWriter::AddTrackWriter(FName TrackName)
	{
		return TrackWriters.Emplace_GetRef(*this, TrackName);
	}

	FTrackWriter& FGeometryCacheConstantTopologyWriter::GetTrackWriter(int32 TrackIndex)
	{
		return TrackWriters[TrackIndex];
	}

	int32 FGeometryCacheConstantTopologyWriter::GetNumTracks() const
	{
		return TrackWriters.Num();
	}

	void FGeometryCacheConstantTopologyWriter::AddMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials)
	{
		if (!Cache.IsValid())
		{
			return;
		}
		Cache->Materials.Append(InMaterials);
	}

	int32 FGeometryCacheConstantTopologyWriter::GetNumMaterials() const
	{
		return Cache.IsValid() ? Cache->Materials.Num() : 0;
	}

	FGeometryCacheConstantTopologyWriter::FTrackWriter::FTrackWriter(FGeometryCacheConstantTopologyWriter& InOwner, FName TrackName)
		: Owner(&InOwner) 
	{
		UGeometryCache* const CachePtr = Owner->Cache.Get();
		if (!CachePtr)
		{
			return;
		}
		const FString BaseName = CachePtr->GetName();
		if (TrackName.IsNone())
		{
			TrackName = MakeUniqueObjectName(CachePtr, UGeometryCacheTrackStreamable::StaticClass(), FName(BaseName + FString(TEXT("_Track"))));
		}
		Track = TStrongObjectPtr(NewObject<UGeometryCacheTrackStreamable>(CachePtr, TrackName, RF_Public));
	}

	FGeometryCacheConstantTopologyWriter::FTrackWriter::~FTrackWriter() = default;

	bool FGeometryCacheConstantTopologyWriter::FTrackWriter::WriteAndClose(TArrayView<TArray<FVector3f>> PositionsToMoveFrom)
	{
		if (!Owner->Cache.IsValid() || !Track.IsValid())
		{
			return false;
		}
		const int32 NumFrames = PositionsToMoveFrom.Num();
		if (NumFrames == 0)
		{
			return false;
		}
		const uint32 NumVertices = PositionsToMoveFrom[0].Num();
		checkSlow(FMath::Max(Indices) < NumVertices);

		TArray<FFrameData> Frames;
		Frames.SetNum(NumFrames);
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			Frames[Frame].Positions = MoveTemp(PositionsToMoveFrom[Frame]);
		}
		return WriteAndClose(Frames);
	}

	bool FGeometryCacheConstantTopologyWriter::FTrackWriter::WriteAndClose(TArrayView<FFrameData> FramesToMoveFrom)
	{
		return WriteAndClose(FramesToMoveFrom, {});
	}

	bool FTrackWriter::WriteAndClose(TArrayView<FFrameData> FramesToMoveFrom, const TArray<FVisibilitySample>& VisibilitySamples)
	{
		if (!Owner->Cache.IsValid() || !Track.IsValid())
		{
			return false;
		}
		const int32 NumFrames = FramesToMoveFrom.Num();
		if (NumFrames == 0)
		{
			return false;
		}
		const uint32 NumVertices = FramesToMoveFrom[0].Positions.Num();
		checkSlow(FMath::Max(Indices) < NumVertices);
		const bool bHasUV0 = !UVs.IsEmpty();
		if (bHasUV0 && UVs.Num() != NumVertices)
		{
			UE_LOGF(LogGeometryCacheWriter, Error, "The number of UVs does not match the number of vertices");
			return false;
		}
		const bool bHasColor0 = !Colors.IsEmpty();
		if (bHasColor0 && Colors.Num() != NumVertices)
		{
			UE_LOGF(LogGeometryCacheWriter, Error, "The number of colors does not match the number of vertices");
			return false;
		}
		const bool bHasImportedVertexNumbers = !ImportedVertexNumbers.IsEmpty();
		if (bHasImportedVertexNumbers && ImportedVertexNumbers.Num() != NumVertices)
		{
			UE_LOGF(LogGeometryCacheWriter, Error, "The number of imported vertex numbers does not match the number of vertices");
			return false;
		}

		UGeometryCacheTrackStreamable* TrackStreamable = CastChecked<UGeometryCacheTrackStreamable>(Track.Get());
		if (!TrackStreamable)
		{
			return false;
		}

		UGeometryCache* const CachePtr = Owner->Cache.Get();
		const FString BaseName = CachePtr->GetName();
		const FName CodecName = MakeUniqueObjectName(CachePtr, UGeometryCacheCodecV1::StaticClass(), FName(BaseName + FString(TEXT("_Codec"))));
		UGeometryCacheCodecV1* Codec = NewObject<UGeometryCacheCodecV1>(Track.Get(), CodecName, RF_Public);
		const FConfig& ConfigRef = Owner->Config;
		Codec->InitializeEncoder(ConfigRef.PositionPrecision, ConfigRef.TextureCoordinatesNumberOfBits);

		constexpr bool bApplyConstantTopologyOptimizations = true;
		constexpr bool bCalculateMotionVectors = false;
		constexpr bool bOptimizeIndexBuffers = false;
		TrackStreamable->BeginCoding(Codec, bApplyConstantTopologyOptimizations, bCalculateMotionVectors, bOptimizeIndexBuffers);

		using namespace Private;
		int32 MaxRecordedFrame = -1;
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			const float Time = static_cast<float>(Frame) / ConfigRef.FPS;
			FFrameData& FrameData = FramesToMoveFrom[Frame];
			if (FrameData.Positions.Num() != NumVertices)
			{
				UE_LOGF(LogGeometryCacheWriter, Error, "The number of vertices at Frame %d does not match that at Frame 0. Finishing up.", Frame);
				break;
			}
			FGeometryCacheMeshData MeshData;
			MeshData.Positions = MoveTemp(FramesToMoveFrom[Frame].Positions);
			MeshData.Indices = Indices;
			MeshData.BoundingBox = GetBoundingBox(MeshData.Positions);

			MeshData.VertexInfo.bHasUV0 = bHasUV0;
			if (bHasUV0)
			{
				MeshData.TextureCoordinates = UVs;
			}
			MeshData.VertexInfo.bHasColor0 = bHasColor0;
			if (bHasColor0)
			{
				MeshData.Colors = Colors;
			}
			MeshData.VertexInfo.bHasImportedVertexNumbers = bHasImportedVertexNumbers;
			if (bHasImportedVertexNumbers)
			{
				MeshData.ImportedVertexNumbers = ImportedVertexNumbers;
			}

			if (bHasUV0)
			{
				if (FrameData.Normals.Num() == NumVertices)
				{
					if (FrameData.TangentsX.Num() == NumVertices)
					{
						MeshData.VertexInfo.bHasTangentX = true;
						MeshData.TangentsX = GetPackedNormals(MoveTemp(FrameData.TangentsX));
					}
					else
					{
						if (!FrameData.TangentsX.IsEmpty())
						{
							UE_LOGF(LogGeometryCacheWriter, Error, "The number of TangentsX at Frame %d does not match that at Frame 0.", Frame);
						}
						MeshData.VertexInfo.bHasTangentX = true;
						MeshData.TangentsX = GetPackedNormals(ComputeTangentsX(MeshData.Indices, MeshData.Positions, FrameData.Normals, UVs));
					}
					MeshData.VertexInfo.bHasTangentZ = true;
					MeshData.TangentsZ = GetPackedNormals(MoveTemp(FrameData.Normals));
				}
				else
				{
					if (!FrameData.Normals.IsEmpty())
					{
						UE_LOGF(LogGeometryCacheWriter, Error, "The number of normals at Frame %d does not match that at Frame 0", Frame);
					}
					MeshData.VertexInfo.bHasTangentZ = true;
					const TArray<FVector3f> Normals = ComputeNormals(MeshData.Indices, MeshData.Positions);
					MeshData.TangentsZ = GetPackedNormals(Normals);
					MeshData.VertexInfo.bHasTangentX = true;
					MeshData.TangentsX = GetPackedNormals(ComputeTangentsX(MeshData.Indices, MeshData.Positions, Normals, UVs));
				}
			}

			MeshData.BatchesInfo = BatchesInfo;

			constexpr bool bConstTopology = true;
			TrackStreamable->AddMeshSample(MeshData, Time, bConstTopology);

			MaxRecordedFrame = Frame;
		}
		check(MaxRecordedFrame >= 0);
		TArray<FMatrix> Mats { FMatrix::Identity, FMatrix::Identity };
		TArray<float> MatTimes { 0.0f, static_cast<float>(MaxRecordedFrame) / ConfigRef.FPS };
		TrackStreamable->SetMatrixSamples(Mats, MatTimes);

		for (const FVisibilitySample& Sample : VisibilitySamples)
		{
			const float Time = static_cast<float>(Sample.FrameIndex) / ConfigRef.FPS;
			TrackStreamable->AddVisibilitySample(Sample.bVisible, Time);
		}
		
		if (ensureAlways(TrackStreamable->EndCoding()))
		{
			CachePtr->AddTrack(TrackStreamable);
			CachePtr->SetFrameStartEnd(0, MaxRecordedFrame);
		}

		Track.Reset();
		return true;
	}

	int32 AddTrackWriterFromSkinnedAsset(FGeometryCacheConstantTopologyWriter& Writer, const USkinnedAsset& Asset)
	{
		constexpr int32 LODIndex = 0;
		return AddTrackWriterFromSkinnedAssetAndMaterials(Writer, Asset, LODIndex, {});
	}

	int32 AddTrackWriterFromSkinnedAssetAndMaterials(FGeometryCacheConstantTopologyWriter& Writer, const USkinnedAsset& Asset, int32 LODIndex, const TArray<TObjectPtr<UMaterialInterface>>& Materials)
	{
		const FSkeletalMeshRenderData* RenderData = Asset.GetResourceForRendering();
		if (!RenderData || !RenderData->LODRenderData.IsValidIndex(LODIndex))
		{
			return INDEX_NONE;
		}
		const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];

		TArray<uint32> Indices;
		LODData.MultiSizeIndexContainer.GetIndexBuffer(Indices);
		using namespace Private;
		const int32 NumVertices = GetNumVertices(LODData);
		const TArray<FVector2f> UVs = GetUV0s(LODData);
		const TArray<FColor> Colors = GetColors(LODData, NumVertices);

		FTrackWriter& TrackWriter = Writer.AddTrackWriter();
		TrackWriter.Indices = Indices;
		TrackWriter.UVs = UVs;
		TrackWriter.Colors = Colors;
		TrackWriter.ImportedVertexNumbers = GetImportedVertexNumbers(Asset, LODIndex);
		const int32 MaterialOffset = Writer.GetNumMaterials();
		TrackWriter.BatchesInfo = GetBatchesInfo(LODData, MaterialOffset);

		TArray<TObjectPtr<UMaterialInterface>> AssetMaterials = GetMaterialInterfaces(Asset);
		if (Materials.Num() == AssetMaterials.Num())
		{
			Writer.AddMaterials(Materials);
		}
		else
		{
			Writer.AddMaterials(AssetMaterials);
		}

		return Writer.GetNumTracks() - 1;
	}

	int32 AddTrackWritersFromTemplateCache(FGeometryCacheConstantTopologyWriter& Writer, const UGeometryCache& TemplateCache)
	{
		int32 NumAddedTracks = 0;
		for (TObjectPtr<UGeometryCacheTrack> Track : TemplateCache.Tracks)
		{
			if (!Track)
			{
				continue;
			}
			FGeometryCacheMeshData MeshData;
			const bool bSuccess = Track->GetMeshDataAtSampleIndex(0, MeshData);
			if (!bSuccess)
			{
				continue;
			}

			FTrackWriter& TrackWriter = Writer.AddTrackWriter(FName(*Track->GetName()));
			TrackWriter.Indices = MeshData.Indices;
			TrackWriter.UVs = MeshData.TextureCoordinates;
			TrackWriter.Colors = MeshData.Colors;
			TrackWriter.ImportedVertexNumbers = MeshData.ImportedVertexNumbers;
			TrackWriter.BatchesInfo = MeshData.BatchesInfo;
			NumAddedTracks++;
		}

		Writer.AddMaterials(TemplateCache.Materials);
		return NumAddedTracks;
	}

	int32 AddTrackWriterFromStaticMesh(FGeometryCacheConstantTopologyWriter& Writer, const UStaticMesh& Asset)
	{
		constexpr int32 LODIndex = 0;
		return AddTrackWriterFromStaticMeshAndMaterials(Writer, Asset, LODIndex, {});
	}

	int32 AddTrackWriterFromStaticMeshAndMaterials(
		FGeometryCacheConstantTopologyWriter& Writer,
		const UStaticMesh& StaticMesh,
		int32 LODIndex,
		const TArray<TObjectPtr<UMaterialInterface>>& Materials)
	{
		FMeshDescription* MeshDescription = nullptr;
#if WITH_EDITORONLY_DATA
		MeshDescription = StaticMesh.GetMeshDescription(LODIndex);
#endif //WITH_EDITORONLY_DATA
		if (!MeshDescription)
		{
			return INDEX_NONE;
		}
		
		FMeshDescriptionToDynamicMesh Converter;
		UE::Geometry::FDynamicMesh3 DynamicMesh;
		Converter.Convert(MeshDescription, DynamicMesh);
		const int VertexCount = DynamicMesh.VertexCount();
		const int TriangleCount = DynamicMesh.TriangleCount();
		// Write data
		FTrackWriter& TrackWriter = Writer.AddTrackWriter();
		TArray<uint32> Indices;
		TArray<FVector2f> UVs;
		TArray<FColor> Colors;

		Indices.SetNumUninitialized(3 * TriangleCount);
		for (int32 TriIdx = 0; TriIdx < TriangleCount; ++TriIdx)
		{
			const UE::Geometry::FIndex3i& Tri = DynamicMesh.GetTriangle(TriIdx);
			for (int32 CornerIdx = 0; CornerIdx < 3; ++CornerIdx)
			{
				Indices[3 * TriIdx + CornerIdx] = Tri[CornerIdx];
			}
		}
		
		TrackWriter.Indices = MoveTemp(Indices);
		UVs.SetNumUninitialized(VertexCount);
		Colors.SetNumUninitialized(VertexCount);
		TrackWriter.ImportedVertexNumbers.SetNum(VertexCount);
		for (int VertexIdx = 0; VertexIdx < VertexCount; ++VertexIdx)
		{
			TrackWriter.ImportedVertexNumbers[VertexIdx] = VertexIdx;
			UVs[VertexIdx] = DynamicMesh.GetVertexUV(VertexIdx);
			Colors[VertexIdx] = FLinearColor(DynamicMesh.GetVertexColor(VertexIdx)).ToFColor(false);
		}
		TrackWriter.UVs = MoveTemp(UVs);
		TrackWriter.Colors = MoveTemp(Colors);

		const int32 MaterialOffset = Writer.GetNumMaterials();
		const TArray<FStaticMaterial>& AssetMaterials = StaticMesh.GetStaticMaterials();
		TArray<TObjectPtr<UMaterialInterface>> MaterialPtrs;
		for (const FStaticMaterial& Mat : AssetMaterials)
		{
			MaterialPtrs.Add(Mat.MaterialInterface);
		}
		Writer.AddMaterials(MaterialPtrs);
		// Only one batch
		FGeometryCacheMeshBatchInfo BatchInfo;
		BatchInfo.StartIndex = 0;
		BatchInfo.NumTriangles = TriangleCount;
		BatchInfo.MaterialIndex = MaterialOffset;
		TrackWriter.BatchesInfo.Add(BatchInfo);
		return Writer.GetNumTracks() - 1;
	}

#endif // WITH_EDITOR
};