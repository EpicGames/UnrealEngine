// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheHelpers.h"
#include "GeometryCacheMeshData.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"

const FName MeshAttribute::VertexInstance::Velocity("Velocity");

namespace UE::GeometryCache::Utils
{
	void GetGeometryCacheMeshDataFromMeshDescription(
		FGeometryCacheMeshData& OutMeshData,
		FMeshDescription& MeshDescription,
		const FMeshDataConversionArguments& Args
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetGeometryCacheMeshDataFromMeshDescription);

		OutMeshData.Positions.Reset();
		OutMeshData.TextureCoordinates.Reset();
		OutMeshData.TangentsX.Reset();
		OutMeshData.TangentsZ.Reset();
		OutMeshData.Colors.Reset();
		OutMeshData.Indices.Reset();

		OutMeshData.MotionVectors.Reset();
		OutMeshData.BatchesInfo.Reset();
		OutMeshData.BoundingBox.Init();

		OutMeshData.VertexInfo.bHasColor0 = true;
		OutMeshData.VertexInfo.bHasTangentX = true;
		OutMeshData.VertexInfo.bHasTangentZ = true;
		OutMeshData.VertexInfo.bHasUV0 = true;

		FStaticMeshAttributes MeshDescriptionAttributes(MeshDescription);

		TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescriptionAttributes.GetVertexPositions();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = MeshDescriptionAttributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = MeshDescriptionAttributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = MeshDescriptionAttributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = MeshDescriptionAttributes.GetVertexInstanceColors();
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = MeshDescriptionAttributes.GetVertexInstanceUVs();

		TVertexInstanceAttributesConstRef<FVector3f>
			VertexInstanceVelocities = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Velocity
			);

		const bool bHasVelocities = Args.bUseVelocitiesAsMotionVectors && VertexInstanceVelocities.IsValid();
		OutMeshData.VertexInfo.bHasMotionVectors = bHasVelocities;

		const int32 NumVertices = MeshDescription.Vertices().Num();
		const int32 NumTriangles = MeshDescription.Triangles().Num();
		const int32 NumMeshDataVertices = NumTriangles * 3;

		OutMeshData.Positions.Reserve(NumVertices);
		OutMeshData.Indices.Reserve(NumMeshDataVertices);
		OutMeshData.TangentsZ.Reserve(NumMeshDataVertices);
		OutMeshData.Colors.Reserve(NumMeshDataVertices);
		OutMeshData.TextureCoordinates.Reserve(NumMeshDataVertices);
		if (bHasVelocities)
		{
			OutMeshData.MotionVectors.Reserve(NumMeshDataVertices);
		}

		const bool bHasImportedVertexNumbers = (NumVertices > 0) && Args.bStoreImportedVertexNumbers;
		if (bHasImportedVertexNumbers)
		{
			OutMeshData.ImportedVertexNumbers.Reserve(NumMeshDataVertices);
		}

		FBox BoundingBox(EForceInit::ForceInitToZero);
		int32 VertexIndex = 0;
		int32 MaterialIndex = Args.MaterialOffset;
		for (FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
		{
			// Skip empty polygon groups
			if (MeshDescription.GetNumPolygonGroupPolygons(PolygonGroupID) == 0)
			{
				continue;
			}

			FGeometryCacheMeshBatchInfo BatchInfo;
			BatchInfo.StartIndex = OutMeshData.Indices.Num();
			BatchInfo.MaterialIndex = MaterialIndex++;

			int32 TriangleCount = 0;
			for (FPolygonID PolygonID : MeshDescription.GetPolygonGroupPolygonIDs(PolygonGroupID))
			{
				for (FTriangleID TriangleID : MeshDescription.GetPolygonTriangles(PolygonID))
				{
					for (FVertexInstanceID VertexInstanceID : MeshDescription.GetTriangleVertexInstances(TriangleID))
					{
						FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
						const FVector3f& Position = VertexPositions[VertexID];
						OutMeshData.Positions.Add(Position);
						BoundingBox += (FVector)Position;

						if (bHasImportedVertexNumbers)
						{
							OutMeshData.ImportedVertexNumbers.Add(VertexID);
						}

						OutMeshData.Indices.Add(VertexIndex++);

						FPackedNormal Normal = VertexInstanceNormals[VertexInstanceID];
						Normal.Vector.W = VertexInstanceBinormalSigns[VertexInstanceID] < 0 ? -127 : 127;
						OutMeshData.TangentsZ.Add(Normal);
						OutMeshData.TangentsX.Add(VertexInstanceTangents[VertexInstanceID]);

						const bool bSRGB = true;
						OutMeshData.Colors.Add(FLinearColor(VertexInstanceColors[VertexInstanceID]).ToFColor(bSRGB));

						// Supporting only one UV channel
						const int32 UVIndex = 0;
						OutMeshData.TextureCoordinates.Add(VertexInstanceUVs.Get(VertexInstanceID, UVIndex));

						if (bHasVelocities)
						{
							FVector3f MotionVector = VertexInstanceVelocities[VertexInstanceID];
							MotionVector *= (-1.f / Args.FramesPerSecond);	 // Velocity is per seconds but we need per frame for motion vectors

							OutMeshData.MotionVectors.Add(MotionVector);
						}
					}

					++TriangleCount;
				}
			}

			OutMeshData.BoundingBox = (FBox3f)BoundingBox;

			BatchInfo.NumTriangles = TriangleCount;
			OutMeshData.BatchesInfo.Add(BatchInfo);
		}
	}
}