// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheMeshData.h"
#include "GeometryCacheModule.h"
#include "UObject/GeometryObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheMeshData)

DECLARE_CYCLE_STAT(TEXT("Deserialize Vertices"), STAT_DeserializeVertices, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Deserialize Indices"), STAT_DeserializeIndices, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Deserialize Metadata"), STAT_DeserializeMetadata, STATGROUP_GeometryCache);


FArchive& operator<<(FArchive& Ar, FGeometryCacheMeshData& Mesh)
{
	Ar.UsingCustomVersion(FGeometryObjectVersion::GUID);

	int32 NumVertices = 0;
		
	if (Ar.IsSaving())
	{
		NumVertices = Mesh.Positions.Num();		
		checkf(Mesh.VertexInfo.bHasMotionVectors == false || Mesh.MotionVectors.Num() == Mesh.Positions.Num(),
			TEXT("Mesh is flagged as having motion vectors but the number of motion vectors does not match the number of vertices"));
	}

	// Serialize metadata first so we can use it later on
	{
		SCOPE_CYCLE_COUNTER(STAT_DeserializeMetadata);
		Ar << Mesh.BoundingBox;
		Ar << Mesh.BatchesInfo;
		Ar << Mesh.VertexInfo;
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_DeserializeVertices);

		Ar << NumVertices;
		if (Ar.IsLoading())
		{
			Mesh.Positions.SetNumUninitialized(NumVertices);

			if (Mesh.VertexInfo.bHasUV0)
			{
				Mesh.TextureCoordinates.SetNumUninitialized(NumVertices);
			}

			if (Mesh.VertexInfo.bHasTangentX)
			{
				Mesh.TangentsX.SetNumUninitialized(NumVertices);
			}

			if (Mesh.VertexInfo.bHasTangentZ)
			{
				Mesh.TangentsZ.SetNumUninitialized(NumVertices);
			}

			if (Mesh.VertexInfo.bHasColor0)
			{
				Mesh.Colors.SetNumUninitialized(NumVertices);
			}

			if (Mesh.VertexInfo.bHasImportedVertexNumbers)
			{
				Mesh.ImportedVertexNumbers.SetNumUninitialized(NumVertices);
			}

			if (Mesh.VertexInfo.bHasMotionVectors)
			{
				Mesh.MotionVectors.SetNumUninitialized(NumVertices);
			}
		}

		if (Mesh.Positions.Num() > 0)
		{				
			Ar.Serialize(&Mesh.Positions[0], Mesh.Positions.Num() * Mesh.Positions.GetTypeSize());

			if (Mesh.VertexInfo.bHasUV0)
			{
				Ar.Serialize(&Mesh.TextureCoordinates[0], Mesh.TextureCoordinates.Num() * Mesh.TextureCoordinates.GetTypeSize());
			}

			if (Mesh.VertexInfo.bHasTangentX)
			{
				Ar.Serialize(&Mesh.TangentsX[0], Mesh.TangentsX.Num() * Mesh.TangentsX.GetTypeSize());
			}

			if (Mesh.VertexInfo.bHasTangentZ)
			{
				Ar.Serialize(&Mesh.TangentsZ[0], Mesh.TangentsZ.Num() * Mesh.TangentsZ.GetTypeSize());
			}

			if (Mesh.VertexInfo.bHasColor0)
			{
				Ar.Serialize(&Mesh.Colors[0], Mesh.Colors.Num() * Mesh.Colors.GetTypeSize());
			}

			if (Mesh.VertexInfo.bHasImportedVertexNumbers)
			{
				Ar.Serialize(&Mesh.ImportedVertexNumbers[0], Mesh.ImportedVertexNumbers.Num() * Mesh.ImportedVertexNumbers.GetTypeSize());
			}
		  
			if (Mesh.VertexInfo.bHasMotionVectors)
			{
				Ar.Serialize(&Mesh.MotionVectors[0], Mesh.MotionVectors.Num()*Mesh.MotionVectors.GetTypeSize());
			}
		}
	}

	{
		// Serializing explicitly here instead of doing Ar << Mesh.Indices
		// makes it about 8 times faster halving the deserialization time of the test mesh
		// so I guess it's worth it here for the little effort it takes
		SCOPE_CYCLE_COUNTER(STAT_DeserializeIndices);
		int32 NumIndices = Mesh.Indices.Num();
		Ar << NumIndices;

		if (Ar.IsLoading())
		{
			Mesh.Indices.Empty(NumIndices);
			Mesh.Indices.AddUninitialized(NumIndices);
		}

		if (Mesh.Indices.Num() > 0)
		{
			Ar.Serialize(&Mesh.Indices[0], Mesh.Indices.Num() * sizeof(uint32));
		}
	}

	return Ar;
}

uint64 FGeometryCacheMeshData::GetHash() const
{
	if (Hash != 0)
	{
		return Hash;
	}

	if (Positions.Num() > 0)
	{
		Hash = CityHash64((char*) Positions.GetData(), Positions.Num() * sizeof(FVector3f));
	}

	if (TextureCoordinates.Num() > 0)
	{
		Hash = CityHash64WithSeed((char*) TextureCoordinates.GetData(), TextureCoordinates.Num() * sizeof(FVector2f), Hash);
	}

	if (TangentsZ.Num() > 0)
	{
		Hash = CityHash64WithSeed((char*) TangentsZ.GetData(), TangentsZ.Num() * sizeof(FPackedNormal), Hash);
	}

	if (BatchesInfo.Num() > 0)
	{
		Hash = CityHash64WithSeed((char*) BatchesInfo.GetData(), BatchesInfo.Num() * sizeof(FGeometryCacheMeshBatchInfo), Hash);
	}

	return Hash;
}

