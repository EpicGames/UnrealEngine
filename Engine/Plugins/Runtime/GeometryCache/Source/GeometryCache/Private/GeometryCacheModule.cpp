// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheModule.h"
#include "CodecV1.h"
#if WITH_EDITOR
#include "GeometryCacheEdModule.h"
#endif // WITH_EDITOR
#include "GeometryCacheStreamingManager.h"
#include "NiagaraGeometryCacheRendererProperties.h"

IMPLEMENT_MODULE(FGeometryCacheModule, GeometryCache)

LLM_DEFINE_TAG(GeometryCache);

void FGeometryCacheModule::StartupModule()
{
	LLM_SCOPE_BYTAG(GeometryCache);

#if WITH_EDITOR
	FGeometryCacheEdModule& Module = FModuleManager::LoadModuleChecked<FGeometryCacheEdModule>(TEXT("GeometryCacheEd"));
#endif

	IGeometryCacheStreamingManager::Register();

	FCodecV1Decoder::InitLUT();

	UNiagaraGeometryCacheRendererProperties::InitCDOPropertiesAfterModuleStartup();
}

void FGeometryCacheModule::ShutdownModule()
{
	IGeometryCacheStreamingManager::Unregister();
}
