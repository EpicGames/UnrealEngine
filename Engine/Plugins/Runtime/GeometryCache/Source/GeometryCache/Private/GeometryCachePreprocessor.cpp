// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCachePreprocessor.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "MeshUtilities.h"
#include "OverlappingCorners.h"
#include "GeometryCacheTrackStreamable.h"

void FCodecGeometryCachePreprocessor::AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious)
{
	FGeometryCacheCodecEncodeArguments Args(
		MeshData,
		SampleTime,
		bSameTopologyAsPrevious
	);

	Track->Codec->CodeFrame(Args);
	Track->Samples.Add(FGeometryCacheTrackStreamableSampleInfo(
		SampleTime,
		(FBox)MeshData.BoundingBox,
		MeshData.Positions.Num(),
		MeshData.Indices.Num()
	));
}

FOptimizeGeometryCachePreprocessor::FOptimizeGeometryCachePreprocessor(FGeometryCachePreprocessor* SetDownStreamProcessor, bool bSetForceSingleOptimization, bool bInOptimizeIndexBuffers)
	: FGeometryCachePreprocessor(SetDownStreamProcessor), NumFramesInBuffer(0), MeshUtilities(FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities")),
	bForceSingleOptimization(bSetForceSingleOptimization),
	bOptimizeIndexBuffers(bInOptimizeIndexBuffers)
{
	BufferedFrames.AddDefaulted(64);
}

FOptimizeGeometryCachePreprocessor::~FOptimizeGeometryCachePreprocessor()
{
	// Flush out the remaining frames
	if (NumFramesInBuffer > 0)
	{
		FlushBufferedFrames();
	}
}

void FOptimizeGeometryCachePreprocessor::AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious)
{
	// Flush out the buffered frames if we have enough of them or if the topo has changed
	if (NumFramesInBuffer == 64 || (bSameTopologyAsPrevious == false && NumFramesInBuffer > 0) )
	{
		FlushBufferedFrames();
	}

	// Append to the list of buffered frames
	BufferedFrames[NumFramesInBuffer].MeshData = MeshData;
	BufferedFrames[NumFramesInBuffer].Time = SampleTime;
	NumFramesInBuffer++;
}

bool FOptimizeGeometryCachePreprocessor::AreIndexedVerticesEqual(int32 IndexBufferIndexA, int32 IndexBufferIndexB)
{
	// They are actually the same index
	if (IndexBufferIndexA == IndexBufferIndexB)
	{
		return true;
	}

	int VertexIndexA = BufferedFrames[0].MeshData.Indices[IndexBufferIndexA];
	int VertexIndexB = BufferedFrames[0].MeshData.Indices[IndexBufferIndexB];

	// There were already pointing to the shame vertex in the unoptimized mesh
	if (VertexIndexA == VertexIndexB)
	{
		return true;
	}

	// Ok do the real equality test across all buffered frames
	for (int32 Frame = 0; Frame < NumFramesInBuffer; Frame++)
	{
		const FVector3f& PositionA = BufferedFrames[Frame].MeshData.Positions[VertexIndexA];
		const FVector3f& PositionB = BufferedFrames[Frame].MeshData.Positions[VertexIndexB];

		if (!PointsEqual(PositionA, PositionB, true))
		{
			return false;
		}

		if (BufferedFrames[Frame].MeshData.TangentsX.Num() > 0)
		{
			// The following are already 8 bit so quantized enough we can do exact equal comparisons
			const FPackedNormal& TangentXA = BufferedFrames[Frame].MeshData.TangentsX[VertexIndexA];
			const FPackedNormal& TangentXB = BufferedFrames[Frame].MeshData.TangentsX[VertexIndexB];

			if (TangentXA != TangentXB)
			{
				return false;
			}
		}

		if (BufferedFrames[Frame].MeshData.TangentsZ.Num() > 0)
		{
			const FPackedNormal& TangentZA = BufferedFrames[Frame].MeshData.TangentsZ[VertexIndexA];
			const FPackedNormal& TangentZB = BufferedFrames[Frame].MeshData.TangentsZ[VertexIndexB];

			if (TangentZA != TangentZB)
			{
				return false;
			}
		}

		if (BufferedFrames[Frame].MeshData.Positions.Num() == BufferedFrames[Frame].MeshData.Colors.Num())
		{
			const FColor& ColorA = BufferedFrames[Frame].MeshData.Colors[VertexIndexA];
			const FColor& ColorB = BufferedFrames[Frame].MeshData.Colors[VertexIndexB];

			if (ColorA != ColorB)
			{
				return false;
			}
		}

		if (BufferedFrames[Frame].MeshData.Positions.Num() == BufferedFrames[Frame].MeshData.TextureCoordinates.Num())
		{
			const FVector2f& UVA = BufferedFrames[Frame].MeshData.TextureCoordinates[VertexIndexA];
			const FVector2f& UVB = BufferedFrames[Frame].MeshData.TextureCoordinates[VertexIndexB];

			if (!UVsEqual(UVA, UVB))
			{
				return false;
			}
		}

		// Motion vectors if we have any
		if (BufferedFrames[Frame].MeshData.Positions.Num() == BufferedFrames[Frame].MeshData.MotionVectors.Num())
		{
			if (!PointsEqual(BufferedFrames[Frame].MeshData.MotionVectors[VertexIndexA], BufferedFrames[Frame].MeshData.MotionVectors[VertexIndexB]))
			{
				return false;
			}
		}
	}

	return true;
}

void FOptimizeGeometryCachePreprocessor::FlushBufferedFrames()
{
	// Do some sanity checking. These at least prevent us from doing out of bounds reads when the topology reporting is broken.
	check(NumFramesInBuffer > 0);
	for (int32 Frame = 1; Frame < NumFramesInBuffer; Frame++)
	{
		checkf(BufferedFrames[Frame].MeshData.Positions.Num() == BufferedFrames[0].MeshData.Positions.Num(), TEXT("Topology was reported as constant but the vertex counts didn't match"));
		checkf(BufferedFrames[Frame].MeshData.Indices.Num() == BufferedFrames[0].MeshData.Indices.Num(), TEXT("Topology was reported as constant but the vertex counts didn't match"));
	}

	// Don't optimize if we already have an optimization and we forced the system to reuse
	// the initial optimization.
	bool bSameTopology = false;
	if (bForceSingleOptimization == false || NewVerticesReordered.Num() == 0)
	{
		// Find the overlapping corners of the first frame. This is not correct final matching list of vertices
		// as we only look at frame 0 but we use it as a quick way to reject vertices which certainly don't match.
		FOverlappingCorners OverlappingCorners;
		
		const TArray<FVector3f>& Positions = BufferedFrames[0].MeshData.Positions;
		const int32 NumVertices = Positions.Num();
		const TArray<uint32>& Indices = BufferedFrames[0].MeshData.Indices;		
		const int32 NumIndices = Indices.Num();
		
		NewIndices.Reset(NumIndices);

		MeshUtilities.FindOverlappingCorners(OverlappingCorners, Positions, Indices, THRESH_POINTS_ARE_SAME);

		// Do a proper matching loop. This matches over all frames and all attributes of the vertices
		TArray<int32> RemappingVertexList;
		RemappingVertexList.SetNumUninitialized(NumVertices);
		TMap<int32, int32> FinalVerts;
		TArray<int32> NewVertices;		

		// Process each face, build vertex buffer and per-section index buffers.
		const int32 NumFaces = NumIndices / 3;
		for (int32 FaceIndex = 0; FaceIndex < NumFaces; FaceIndex++)
		{
			int32 VertexIndices[3];

			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				int32 WedgeIndex = FaceIndex * 3 + CornerIndex;
				const TArray<int32>& SharedVertices = OverlappingCorners.FindIfOverlapping(WedgeIndex);

				// Determine the new index of this vertex in the optimized vertex list
				int32 Index = INDEX_NONE;
				for (int32 SharedId = 0; SharedId < SharedVertices.Num(); SharedId++)
				{
					if (SharedVertices[SharedId] >= WedgeIndex)
					{
						// the verts beyond me haven't been placed yet, so these duplicates are not relevant
						break;
					}

					// Look at the vertices sharing position at frame 0 to see if they match with this vert
					const int32 *Location = FinalVerts.Find(SharedVertices[SharedId]);
					if (Location != NULL // if this is null the vertex was already shared and we'll visit it shared copy
						&& AreIndexedVerticesEqual(WedgeIndex, SharedVertices[SharedId])) // Check if we can share
					{
						Index = *Location;
						break;
					}
				}

				// No matching vert found, allocate a new vertex
				if (Index == INDEX_NONE)
				{
					Index = NewVertices.Add(Indices[WedgeIndex]);
					FinalVerts.Add(WedgeIndex, Index);
				}
				VertexIndices[CornerIndex] = Index;
			}

			// Reject degenerate triangles. These got mapped to the same vertex in all frames
			/* This is disabled as it would require redoing the batch info...
			 if (VertexIndices[0] == VertexIndices[1]
				|| VertexIndices[1] == VertexIndices[2]
				|| VertexIndices[0] == VertexIndices[2])
			{
				continue;
			}*/

			for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++)
			{
				/*SectionIndices.Add(VertexIndices[CornerIndex]);*/
				NewIndices.Add(VertexIndices[CornerIndex]);
			}
		}

		if (bOptimizeIndexBuffers)
		{
			MeshUtilities.CacheOptimizeIndexBuffer(NewIndices);
		}

		// Enable this to stress test for topology changes
		// it randomly reshuffles the triangles in the mesh
		#if 0
		{
			int32 NumTriangles = NewIndices.Num() / 3;
			for (int Idx = NumTriangles - 1; Idx >= 0; --Idx)
			{
				int32 SwapIdx = FMath::Rand() % (Idx + 1);

				for (int32 VertIdx = 0; VertIdx < 3; VertIdx++)
				{
					int32 temp = NewIndices[Idx * 3 + VertIdx];
					NewIndices[Idx * 3 + VertIdx] = NewIndices[SwapIdx * 3 + VertIdx];
					NewIndices[SwapIdx * 3 + VertIdx] = temp;
				}
			}
		}
		#endif

		// The optimizing above may have reordered the number the vertices are visited in. For optimal
		// index buffer compression we now reorder the vertices so the first accessed vertices
		// also get the lower vertex id's
		TMap<int32, int32> VertexMapping;
		int32 NextVertexId = 0;
		for (int32 IndexId = 0; IndexId < NewIndices.Num(); IndexId++)
		{
			// Did we encounter and thus re-order this vertex previously?
			int32 *ReorderedVertIdx = VertexMapping.Find(NewIndices[IndexId]);
			if (ReorderedVertIdx == nullptr)
			{
				// No? so allocate the next lowest available vertex id to this vert
				VertexMapping.Add(NewIndices[IndexId], NextVertexId);
				NewIndices[IndexId] = NextVertexId;
				NextVertexId++;
			}
			else
			{
				NewIndices[IndexId] = *ReorderedVertIdx;
			}
		}

		TArray<int32> OldVerticesReordered(MoveTemp(NewVerticesReordered));

		// We updated the index buffer inline above but still need to update NewVertices ordering
		NewVerticesReordered.SetNumUninitialized(NewVertices.Num());
		bSameTopology = OldVerticesReordered.Num() == NewVerticesReordered.Num();

		for (const auto& Pair : VertexMapping)
		{
			int32 RemappedVertId = Pair.Value;
			NewVerticesReordered[RemappedVertId] = NewVertices[Pair.Key];
			bSameTopology = bSameTopology && (NewVerticesReordered[RemappedVertId] == OldVerticesReordered[RemappedVertId]);
		}
	}

	for (int32 Frame = 0; Frame < NumFramesInBuffer; Frame++)
	{
		FGeometryCacheMeshData &OldMesh = BufferedFrames[Frame].MeshData;
		FGeometryCacheMeshData NewMesh;
		NewMesh.BatchesInfo = OldMesh.BatchesInfo;
		NewMesh.BoundingBox = OldMesh.BoundingBox;
		NewMesh.VertexInfo = OldMesh.VertexInfo;
		NewMesh.Indices = NewIndices;

		const bool bHasTangentsX = OldMesh.TangentsX.Num() > 0;
		const bool bHasTangentsZ = OldMesh.TangentsZ.Num() > 0;
		const bool bHasImportedVertexNumbers = OldMesh.ImportedVertexNumbers.Num() > 0;

		NewMesh.Positions.SetNumUninitialized(NewVerticesReordered.Num());

		if (bHasImportedVertexNumbers)
		{
			NewMesh.ImportedVertexNumbers.SetNumUninitialized(NewVerticesReordered.Num());
		}

		if (bHasTangentsX)
		{
			NewMesh.TangentsX.SetNumUninitialized(NewVerticesReordered.Num());
		}

		if (bHasTangentsZ)
		{
			NewMesh.TangentsZ.SetNumUninitialized(NewVerticesReordered.Num());
		}

		if (NewMesh.VertexInfo.bHasColor0)
		{
			NewMesh.Colors.SetNumUninitialized(NewVerticesReordered.Num());
		}
		
		if ( NewMesh.VertexInfo.bHasUV0)
		{
			NewMesh.TextureCoordinates.SetNumUninitialized(NewVerticesReordered.Num());
		}

		if (NewMesh.VertexInfo.bHasMotionVectors)
		{
			NewMesh.MotionVectors.SetNumUninitialized(NewVerticesReordered.Num());
		}

		FBox3f Bounds(EForceInit::ForceInit);
		for (int32 i = 0; i < NewMesh.Positions.Num(); i++)
		{
			NewMesh.Positions[i] = OldMesh.Positions[NewVerticesReordered[i]];
			if (bHasImportedVertexNumbers)
			{
				NewMesh.ImportedVertexNumbers[i] = OldMesh.ImportedVertexNumbers[NewVerticesReordered[i]];
			}
			if (bHasTangentsX)
			{
				NewMesh.TangentsX[i] = OldMesh.TangentsX[NewVerticesReordered[i]];
			}
			if (bHasTangentsZ)
			{
				NewMesh.TangentsZ[i] = OldMesh.TangentsZ[NewVerticesReordered[i]];
			}
			if (NewMesh.VertexInfo.bHasColor0)
			{
				NewMesh.Colors[i] = OldMesh.Colors[NewVerticesReordered[i]];
			}
			if (NewMesh.VertexInfo.bHasUV0)
			{
				NewMesh.TextureCoordinates[i] = OldMesh.TextureCoordinates[NewVerticesReordered[i]];
			}
			
			if (NewMesh.VertexInfo.bHasMotionVectors)
			{
				NewMesh.MotionVectors[i] = OldMesh.MotionVectors[NewVerticesReordered[i]];
				if (NewMesh.MotionVectors[i].ContainsNaN())
				{
					NewMesh.MotionVectors[i] = i > 0 ? NewMesh.MotionVectors[i - 1] : FVector3f::ZeroVector;
				}
			}
			Bounds += NewMesh.Positions[i];
		}

		NewMesh.BoundingBox = Bounds;
		/*
		FGeometryCacheCodecEncodeArguments Args(
			NewMesh,
			BufferedFrames[Frame].Time,
			(Frame == 0 && !bForceSingleOptimization) ? false : true // We only ever buffer frames with the same topo so we can just look at the frame id here
		);

		Track->Codec->CodeFrame(Args);
		Track->Samples.Add(FGeometryCacheTrackStreamableSampleInfo(
			BufferedFrames[Frame].Time,
			NewMesh.BoundingBox,
			NewMesh.Vertices.Num(),
			NewMesh.Indices.Num()
		));*/

		bool bIsSameTopologyAsPrevious = (Frame == 0 && !bForceSingleOptimization) ? bSameTopology : true;
		DownStreamProcessor->AddMeshSample(NewMesh, BufferedFrames[Frame].Time, bIsSameTopologyAsPrevious);
	}

	NumFramesInBuffer = 0;
}

FExplicitMotionVectorGeometryCachePreprocessor::~FExplicitMotionVectorGeometryCachePreprocessor()
{
	// Flush out the last remaining frame
	if (bHasPreviousFrame)
	{
		// add zero motion vectors if none yet
		if (PreviousFrame.MotionVectors.Num() != PreviousFrame.Positions.Num())
		{
			PreviousFrame.MotionVectors.SetNumZeroed(PreviousFrame.Positions.Num());
		}
		PreviousFrame.VertexInfo.bHasMotionVectors = PreviousFrame.MotionVectors.Num() > 0;

		DownStreamProcessor->AddMeshSample(PreviousFrame, PreviousFrameTime, bPreviousTopologySame);
	}
}

void FExplicitMotionVectorGeometryCachePreprocessor::AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious)
{
	if (bHasPreviousFrame)
	{
		// Same topology and no explicit motion vectors yet, generate them
		if (bSameTopologyAsPrevious && PreviousFrame.MotionVectors.Num() != PreviousFrame.Positions.Num())
		{
			PreviousFrame.MotionVectors.SetNumUninitialized(PreviousFrame.Positions.Num());
			for (int32 VertID = 0; VertID < PreviousFrame.Positions.Num(); VertID++)
			{
				PreviousFrame.MotionVectors[VertID] = MeshData.Positions[VertID] - PreviousFrame.Positions[VertID];
			}
		}
		// Not compatible with previous frame, impossible to generate anything  we just emit zero MV's here for completeness so the mesh is guaranteed to have them
		else if (PreviousFrame.MotionVectors.Num() != PreviousFrame.Positions.Num())
		{
			PreviousFrame.MotionVectors.SetNumZeroed(PreviousFrame.Positions.Num());
		}

		PreviousFrame.VertexInfo.bHasMotionVectors = PreviousFrame.MotionVectors.Num() > 0;
		DownStreamProcessor->AddMeshSample(PreviousFrame, PreviousFrameTime, bPreviousTopologySame);
	}

	// Save this frame for the next frame
	bHasPreviousFrame = true;
	PreviousFrameTime = SampleTime;
	PreviousFrame = MeshData;
	bPreviousTopologySame = bSameTopologyAsPrevious;
}

#endif // WITH_EDITOR
