// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "GeometryCacheMeshData.h"

class UGeometryCacheTrackStreamable;

class IMeshUtilities;

/**
* A generic object that preprocesses frames coming from the geometry cache importer and transforms them. The processed frames are then passed on to 
* another preprocessor. The final preprocessor in the chain then calls the codec.
*/
class FGeometryCachePreprocessor
{
public:

	/**
	* @param DownStreamProcessor The processor to pass processed results to. The processor will be owned by this object and deleted then it is destroyed.
	*/
	FGeometryCachePreprocessor(FGeometryCachePreprocessor* SetDownStreamProcessor) : DownStreamProcessor(SetDownStreamProcessor) {}
	virtual ~FGeometryCachePreprocessor() {
		if (DownStreamProcessor != nullptr)
		{
			delete DownStreamProcessor;
		}
	}

	virtual void AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious) = 0;

protected:
	FGeometryCachePreprocessor* DownStreamProcessor;
};

class FCodecGeometryCachePreprocessor : public FGeometryCachePreprocessor
{
public:
	FCodecGeometryCachePreprocessor(UGeometryCacheTrackStreamable *SetTrack) : FGeometryCachePreprocessor(NULL), Track(SetTrack) {}
	virtual void AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious) override;
protected:
	UGeometryCacheTrackStreamable *Track;
};

/**
 * This is a class that is shared by all codecs and does some things like mesh sanitization and other preprocessing tasks before
 * handling the actual mesh to the codecs. This allows common preprocessing tasks to be shared across codecs and allows the codecs
 * to assume a certain guaranteed level of mesh sanitization without putting too much burden on the individual importers.
 */
class FOptimizeGeometryCachePreprocessor : public FGeometryCachePreprocessor
{
public:

	/**
	 * @param SetDownStreamProcessor The downstream processor to pass results to
	 * @param bSetForceSingleOptimization Only run the optimization phase of the preprocessor once. With generic meshes optimizing without looking at the
	 *			whole mesh can give some invalid results (eg. triangles drifting apart, changing smoothing groups for normals in animations, ....)
	 *			this flag tells the system it's ok to ignore all this and just run the optimization once then reuse the results for all frames.
	 *			This may lead to strange artifacts such as parts drifting apart still being connected by triangles, smoothing group creases not appearing, ...
	 *			but in for well behaved meshes it's probably going to be almost invisible.
	 */
	FOptimizeGeometryCachePreprocessor(FGeometryCachePreprocessor* SetDownStreamProcessor, bool bSetForceSingleOptimization, bool bInOptimizeIndexBuffers);
	~FOptimizeGeometryCachePreprocessor() override;

	void AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious) override;

protected:

	struct FBufferedFrame
	{
		float Time;
		FGeometryCacheMeshData MeshData;
	};

	TArray<FBufferedFrame> BufferedFrames;
	int NumFramesInBuffer;

	void FlushBufferedFrames();
	bool AreIndexedVerticesEqual(int32 IndexBufferIndexA, int32 IndexBufferIndexB);

	// We have to cache the mesh utilities here since we can't load modules from other threads and AddMeshSample is possibly
	// called from worker threads
	IMeshUtilities& MeshUtilities;

	bool bForceSingleOptimization;
	bool bOptimizeIndexBuffers;
	
	// These contain the optimization results and are cached when bForceSingleOptimization is forced
	TArray<uint32> NewIndices;
	TArray<int32> NewVerticesReordered;

};


/**
* Adds explicit motion vectors to a mesh by taking the difference between consecutive frames. Any frames which already have explicit motion
* vectors specified will use these existing motion vectors instead of deriving them.
*/
class FExplicitMotionVectorGeometryCachePreprocessor : public FGeometryCachePreprocessor
{
public:

	FExplicitMotionVectorGeometryCachePreprocessor(FGeometryCachePreprocessor* SetDownStreamProcessor) : FGeometryCachePreprocessor(SetDownStreamProcessor), bHasPreviousFrame(false) {}
	~FExplicitMotionVectorGeometryCachePreprocessor() override;

	void AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious) override;

protected:
	FGeometryCacheMeshData PreviousFrame;
	float PreviousFrameTime;
	bool bHasPreviousFrame;
	bool bPreviousTopologySame;
	void FlushBufferedFrame();
};

#endif // WITH_EDITOR
