// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheSceneProxy.h"
#include "Async/ParallelFor.h"
#include "Components/BrushComponent.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCache.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheModule.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "PrimitiveViewRelevance.h"
#include "RayTracingInstance.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PrimitiveUniformShaderParametersBuilder.h"

#include "SceneView.h"

#if INTEL_ISPC
#include "GeometryCacheSceneProxy.ispc.generated.h"
#endif

#if INTEL_ISPC
static_assert(sizeof(uint32) == sizeof(FPackedNormal), "sizeof(uint32) != sizeof(FPackedNormal)");
static_assert(sizeof(ispc::FVector2f) == sizeof(FVector2f), "sizeof(ispc::FVector2f) != sizeof(FVector2f)");
static_assert(sizeof(ispc::FVector3f) == sizeof(FVector3f), "sizeof(ispc::FVector3f) != sizeof(FVector3f)");

#if !UE_BUILD_SHIPPING
bool GGeometryCacheSceneProxyUseIspc = GEOMETRY_CACHE_SCENE_PROXY_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarGeometryCacheSceneProxyUseIspc(
	TEXT("r.GeometryCacheSceneProxy.ISPC"),
	GGeometryCacheSceneProxyUseIspc,
	TEXT("When enabled GeometryCacheSceneProxy will use ISPC if appropriate."),
	ECVF_Default
);
#endif

#endif

DECLARE_CYCLE_STAT(TEXT("Gather Mesh Elements"), STAT_GeometryCacheSceneProxy_GetMeshElements, STATGROUP_GeometryCache);
DECLARE_DWORD_COUNTER_STAT(TEXT("Triangle Count"), STAT_GeometryCacheSceneProxy_TriangleCount, STATGROUP_GeometryCache);
DECLARE_DWORD_COUNTER_STAT(TEXT("Batch Count"), STAT_GeometryCacheSceneProxy_MeshBatchCount, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Vertex Buffer Update"), STAT_VertexBufferUpdate, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Index Buffer Update"), STAT_IndexBufferUpdate, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Buffer Update Task"), STAT_BufferUpdateTask, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("InterpolateFrames"), STAT_InterpolateFrames, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("InterpolateFramesSIMDPositions"), STAT_InterpolateFramesSIMDPositions, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("InterpolateFramesSIMDMotionVectors"), STAT_InterpolateFramesSIMDMotionVectors, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("InterpolateFramesSIMDTangents"), STAT_InterpolateFramesSIMDTangents, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("InterpolateFramesSIMDUVs"), STAT_InterpolateFramesSIMDUVs, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("InterpolateFramesSIMDColors"), STAT_InterpolateFramesSIMDColors, STATGROUP_GeometryCache);

static TAutoConsoleVariable<int32> CVarOffloadUpdate(
	TEXT("GeometryCache.OffloadUpdate"),
	0,
	TEXT("Offloat some updates from the render thread to the workers & RHI threads."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarInterpolateFrames(
	TEXT("GeometryCache.InterpolateFrames"),
	1,
	TEXT("Interpolate between geometry cache frames (if topology allows this)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingGeometryCache(
	TEXT("r.RayTracing.Geometry.GeometryCache"),
	1,
	TEXT("Include geometry cache primitives in ray tracing effects (default = 1 (geometry cache enabled in ray tracing))"));

/**
* All vertex information except the position.
*/
struct FNoPositionVertex
{
	FVector2f TextureCoordinate[MAX_STATIC_TEXCOORDS];
	FPackedNormal TangentX;
	FPackedNormal TangentZ;
	FColor Color;
};

FGeometryCacheSceneProxy::FGeometryCacheSceneProxy(UGeometryCacheComponent* Component) 
: FGeometryCacheSceneProxy(Component, [this]() { return new FGeomCacheTrackProxy(GetScene().GetFeatureLevel()); })
{
}

FGeometryCacheSceneProxy::FGeometryCacheSceneProxy(UGeometryCacheComponent* Component, TFunction<FGeomCacheTrackProxy*()> TrackProxyCreator)
: FPrimitiveSceneProxy(Component)
, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
, CreateTrackProxy(TrackProxyCreator)
{
	const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();

	Time = Component->GetAnimationTime();
	bLooping = Component->IsLooping();
	bExtrapolateFrames = Component->IsExtrapolatingFrames();
	bInterpolateFrames = Component->IsInterpolatingFrames();
	bAlwaysHasVelocity = true;
	PlaybackSpeed = (Component->IsPlaying()) ? Component->GetPlaybackSpeed() : 0.0f;
	MotionVectorScale = Component->GetMotionVectorScale();
	bOverrideWireframeColor = Component->GetOverrideWireframeColor();
	WireframeOverrideColor = Component->GetWireframeOverrideColor();
	
	UpdatedFrameNum = 0;

	EnableGPUSceneSupportFlags();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bCanSkipRedundantTransformUpdates = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// All tracks use the same array of materials since the MaterialIndex in FGeometryCacheMeshBatchInfo 
	// is an index into the global material array UGeometryCache::Materials
	TArray<UMaterialInterface*> Materials;
	for (UMaterialInterface* Material : Component->GetMaterials())
	{
		if (Material == nullptr || !Material->CheckMaterialUsage_Concurrent(EMaterialUsage::MATUSAGE_GeometryCache))
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		Materials.Add(Material);
	}	
	
	// Copy each section
	const int32 NumTracks = Component->TrackSections.Num();
	Tracks.Reserve(NumTracks);
	for (int32 TrackIdx = 0; TrackIdx < NumTracks; TrackIdx++)
	{
		FTrackRenderData& SrcSection = Component->TrackSections[TrackIdx];
		UGeometryCacheTrack* CurrentTrack = Component->GeometryCache->Tracks[TrackIdx];

		const FGeometryCacheTrackSampleInfo& SampleInfo = CurrentTrack->GetSampleInfo(Time, bLooping);

		FGeomCacheTrackProxy* NewSection = CreateTrackProxy();

		NewSection->Track = CurrentTrack;
		NewSection->WorldMatrix = SrcSection.Matrix;
		NewSection->FrameIndex = -1;
		NewSection->UploadedSampleIndex = -1;
		NewSection->NextFrameIndex = -1;
		NewSection->PreviousFrameIndex = -1;
		NewSection->InterpolationFactor = 0.f;
		NewSection->PreviousInterpolationFactor = 0.f;
		NewSection->SubframeInterpolationFactor = 1.f;
		NewSection->NextFrameMeshData = nullptr;
		NewSection->bResourcesInitialized = false;

		if (SampleInfo.NumVertices > 0)
		{
			ENQUEUE_RENDER_COMMAND(FGeometryCacheInitResources)([NewSection, NumVertices = SampleInfo.NumVertices, NumIndices = SampleInfo.NumIndices] (FRHICommandListBase& RHICmdList)
			{
				NewSection->InitRenderResources(RHICmdList, NumVertices, NumIndices);
			});
		}

		// Grab materials
		int32 Dummy = -1;
		NewSection->MeshData = new FGeometryCacheMeshData();
		NewSection->UpdateMeshData(Time, bLooping, Dummy, *NewSection->MeshData);
		NewSection->NextFrameMeshData = new FGeometryCacheMeshData();
		NewSection->bNextFrameMeshDataSelected = false;

		NewSection->Materials = Materials;

		// Save ref to new section
		Tracks.Add(NewSection);
	}

	// Update at least once after the scene proxy has been constructed
	// Otherwise it is invisible until animation starts
	FGeometryCacheSceneProxy* SceneProxy = this;
	ENQUEUE_RENDER_COMMAND(FGeometryCacheUpdateAnimation)(
		[SceneProxy](FRHICommandListImmediate& RHICmdList)
	{
		SceneProxy->FrameUpdate(RHICmdList);
	});

	if (IsRayTracingEnabled())
	{
#if RHI_RAYTRACING
		{
			RayTracingDebugName = Component->GetFName();
			ENQUEUE_RENDER_COMMAND(FGeometryCacheInitRayTracingGeometry)(
			[SceneProxy](FRHICommandListImmediate& RHICmdList)
			{
				SceneProxy->InitRayTracing(RHICmdList);
			});
		}
#endif
	}
}

FGeometryCacheSceneProxy::~FGeometryCacheSceneProxy()
{
	for (FGeomCacheTrackProxy* Section : Tracks)
	{
		if (Section != nullptr)
		{
			Section->TangentXBuffer.ReleaseResource();
			Section->TangentZBuffer.ReleaseResource();
			Section->TextureCoordinatesBuffer.ReleaseResource();
			Section->ColorBuffer.ReleaseResource();
			Section->IndexBuffer.ReleaseResource();
			Section->VertexFactory.ReleaseResource();
			Section->PositionBuffers[0].ReleaseResource();
			Section->PositionBuffers[1].ReleaseResource();
#if RHI_RAYTRACING
			Section->RayTracingGeometry.ReleaseResource();
#endif
			delete Section->MeshData;
			if (Section->NextFrameMeshData != nullptr)
				delete Section->NextFrameMeshData;
			delete Section;
		}
	}
	Tracks.Empty();
}

class FGeometryCacheVertexFactoryUserDataWrapper : public FOneFrameResource
{
public:
	FGeometryCacheVertexFactoryUserData Data;
};

static float OneOver255 = 1.0f / 255.0f;

// Avoid converting from 8 bit normalized to float and back again.
inline FPackedNormal InterpolatePackedNormal(const FPackedNormal& A, const FPackedNormal& B, int32 ScaledFactor, int32 OneMinusScaledFactor)
{
	FPackedNormal result;
	result.Vector.X = static_cast<int8>(static_cast<float>(A.Vector.X * OneMinusScaledFactor + B.Vector.X * ScaledFactor) * OneOver255);
	result.Vector.Y = static_cast<int8>(static_cast<float>(A.Vector.Y * OneMinusScaledFactor + B.Vector.Y * ScaledFactor) * OneOver255);
	result.Vector.Z = static_cast<int8>(static_cast<float>(A.Vector.Z * OneMinusScaledFactor + B.Vector.Z * ScaledFactor) * OneOver255);
	result.Vector.W = static_cast<int8>(static_cast<float>(A.Vector.W * OneMinusScaledFactor + B.Vector.W * ScaledFactor) * OneOver255);
	return result;
}

// Avoid converting from 8 bit normalized to float and back again.
inline FColor InterpolatePackedColor(const FColor& A, const FColor& B, int32 ScaledFactor, int32 OneMinusScaledFactor)
{
	FColor result;
	result.R = static_cast<uint8>(static_cast<float>(A.R * OneMinusScaledFactor + B.R * ScaledFactor) * OneOver255);
	result.G = static_cast<uint8>(static_cast<float>(A.G * OneMinusScaledFactor + B.G * ScaledFactor) * OneOver255);
	result.B = static_cast<uint8>(static_cast<float>(A.B * OneMinusScaledFactor + B.B * ScaledFactor) * OneOver255);
	result.A = static_cast<uint8>(static_cast<float>(A.A * OneMinusScaledFactor + B.A * ScaledFactor) * OneOver255);
	return result;
}

SIZE_T FGeometryCacheSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FGeometryCacheSceneProxy::CreateMeshBatch(
	FRHICommandListBase& RHICmdList,
	const FGeomCacheTrackProxy* TrackProxy,
	const FGeometryCacheMeshBatchInfo& BatchInfo,
	FGeometryCacheVertexFactoryUserDataWrapper& UserDataWrapper,
	FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer,
	FMeshBatch& Mesh) const
{
	FGeometryCacheVertexFactoryUserData& UserData = UserDataWrapper.Data;

	UserData.MeshExtension = FVector3f::OneVector;
	UserData.MeshOrigin = FVector3f::ZeroVector;

	const bool bHasMotionVectors = (
		TrackProxy->MeshData->VertexInfo.bHasMotionVectors &&
		TrackProxy->NextFrameMeshData->VertexInfo.bHasMotionVectors &&
		TrackProxy->MeshData->Positions.Num() == TrackProxy->MeshData->MotionVectors.Num())
		&& (TrackProxy->NextFrameMeshData->Positions.Num() == TrackProxy->NextFrameMeshData->MotionVectors.Num());

	if (!bHasMotionVectors)
	{
		const float PreviousPositionScale = (GFrameNumber <= UpdatedFrameNum) ? 1.f : 0.f;
		UserData.MotionBlurDataExtension = FVector3f::OneVector * PreviousPositionScale;
		UserData.MotionBlurDataOrigin = FVector3f::ZeroVector;
		UserData.MotionBlurPositionScale = 1.f - PreviousPositionScale;
	}
	else
	{
		UserData.MotionBlurDataExtension = FVector3f::OneVector * PlaybackSpeed * TrackProxy->SubframeInterpolationFactor;
		UserData.MotionBlurDataOrigin = FVector3f::ZeroVector;
		UserData.MotionBlurPositionScale = 1.0f;
	}

	if (IsRayTracingEnabled())
	{
		// No vertex manipulation is allowed in the vertex shader
		// Otherwise we need an additional compute shader pass to execute the vertex shader and dump to a staging buffer
		check(UserData.MeshExtension == FVector3f::OneVector);
		check(UserData.MeshOrigin == FVector3f::ZeroVector);
	}

	UserData.PositionBuffer = &TrackProxy->PositionBuffers[TrackProxy->CurrentPositionBufferIndex % 2];
	UserData.MotionBlurDataBuffer = &TrackProxy->PositionBuffers[(TrackProxy->CurrentPositionBufferIndex + 1) % 2];

	FGeometryCacheVertexFactoryUniformBufferParameters UniformBufferParameters;

	UniformBufferParameters.MeshOrigin = UserData.MeshOrigin;
	UniformBufferParameters.MeshExtension = UserData.MeshExtension;
	UniformBufferParameters.MotionBlurDataOrigin = UserData.MotionBlurDataOrigin;
	UniformBufferParameters.MotionBlurDataExtension = UserData.MotionBlurDataExtension;
	UniformBufferParameters.MotionBlurPositionScale = UserData.MotionBlurPositionScale;

	UserData.UniformBuffer = FGeometryCacheVertexFactoryUniformBufferParametersRef::CreateUniformBufferImmediate(UniformBufferParameters, UniformBuffer_SingleFrame);
	TrackProxy->VertexFactory.CreateManualVertexFetchUniformBuffer(RHICmdList, UserData.PositionBuffer, UserData.MotionBlurDataBuffer, UserData);

	// Draw the mesh.
	FMeshBatchElement& BatchElement = Mesh.Elements[0];
	BatchElement.IndexBuffer = &TrackProxy->IndexBuffer;
	Mesh.VertexFactory = &TrackProxy->VertexFactory;
	Mesh.SegmentIndex = 0;

	FPrimitiveUniformShaderParametersBuilder Builder;
	BuildUniformShaderParameters(Builder);
	Builder.LocalToWorld(TrackProxy->WorldMatrix * GetLocalToWorld());
	DynamicPrimitiveUniformBuffer.Set(RHICmdList, Builder);

	BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

	const FGeometryCacheMeshData* MeshData = TrackProxy->bNextFrameMeshDataSelected ? TrackProxy->NextFrameMeshData : TrackProxy->MeshData;

	BatchElement.FirstIndex = BatchInfo.StartIndex;
	BatchElement.NumPrimitives = BatchInfo.NumTriangles;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = MeshData->Positions.Num() - 1;
	BatchElement.VertexFactoryUserData = &UserDataWrapper.Data;
	Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
	Mesh.Type = PT_TriangleList;
	Mesh.DepthPriorityGroup = SDPG_World;
	Mesh.bCanApplyViewModeOverrides = false;
}

#if WITH_EDITOR
HHitProxy* FGeometryCacheSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	// Add a default hit proxy to handle cases where the number of batches changes during the animation,
	// including when the initial frame has no mesh data
	HHitProxy* DefaultHitProxy = FPrimitiveSceneProxy::CreateHitProxies(Component, OutHitProxies);

	if (Component->GetOwner() && Tracks.Num() > 0)
	{
		int32 SectionIndex = 0;
		for (const FGeomCacheTrackProxy* Track : Tracks)
		{
			if (Track->MeshData)
			{
				for (const FGeometryCacheMeshBatchInfo& BatchInfo : Track->MeshData->BatchesInfo)
				{
					HHitProxy* ActorHitProxy;

					int32 MaterialIndex = BatchInfo.MaterialIndex;
					if (Component->GetOwner()->IsA(ABrush::StaticClass()) && Component->IsA(UBrushComponent::StaticClass()))
					{
						ActorHitProxy = new HActor(Component->GetOwner(), Component, HPP_Wireframe, SectionIndex, MaterialIndex);
					}
					else
					{
						ActorHitProxy = new HActor(Component->GetOwner(), Component, Component->HitProxyPriority, SectionIndex, MaterialIndex);
					}

					OutHitProxies.Add(ActorHitProxy);
					++SectionIndex;
				}
			}
		}
	}

	HitProxyIds.SetNumUninitialized(OutHitProxies.Num());
	for (int32 Index = 0; Index < HitProxyIds.Num(); ++Index)
	{
		HitProxyIds[Index] = OutHitProxies[Index]->Id;
	}

	return DefaultHitProxy;
}
#endif

void FGeometryCacheSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	SCOPE_CYCLE_COUNTER(STAT_GeometryCacheSceneProxy_GetMeshElements);
	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	// Set up wire frame material (if needed)
	const bool bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;

	FColoredMaterialRenderProxy* WireframeMaterialInstance = nullptr;
	if (bWireframe)
	{
		const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;
		const bool bActorColorationEnabled = EngineShowFlags.ActorColoration;

		const FLinearColor WireColor = bOverrideWireframeColor ? WireframeOverrideColor : GetWireframeColor();
		const FLinearColor ViewWireframeColor(bActorColorationEnabled ? GetPrimitiveColor() : WireColor);

		WireframeMaterialInstance = new FColoredMaterialRenderProxy(
			GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : nullptr,
			GetSelectionColor(ViewWireframeColor, !(GIsEditor && EngineShowFlags.Selection) || IsSelected(), IsHovered(), false)
		);

		Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);
	}
	
	const bool bVisible = [&Views, VisibilityMap]()
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				return true;
			}
		}
		return false;
	}();

	if (bVisible)
	{
		// Iterate over all batches in all tracks and add them to all the relevant views	
		for (const FGeomCacheTrackProxy* TrackProxy : Tracks)
		{
			const FVisibilitySample& VisibilitySample = TrackProxy->GetVisibilitySample(Time, bLooping);
			if (!VisibilitySample.bVisibilityState)
			{
				continue;
			}

			const FGeometryCacheMeshData* MeshData = TrackProxy->bNextFrameMeshDataSelected ? TrackProxy->NextFrameMeshData : TrackProxy->MeshData;
			const int32 NumBatches = MeshData->BatchesInfo.Num();

			for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
			{
				const FGeometryCacheMeshBatchInfo& BatchInfo = MeshData->BatchesInfo[BatchIndex];

				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if (VisibilityMap & (1 << ViewIndex))
					{
						FMeshBatch& MeshBatch = Collector.AllocateMesh();

						FGeometryCacheVertexFactoryUserDataWrapper& UserDataWrapper = Collector.AllocateOneFrameResource<FGeometryCacheVertexFactoryUserDataWrapper>();
						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						CreateMeshBatch(RHICmdList, TrackProxy, BatchInfo, UserDataWrapper, DynamicPrimitiveUniformBuffer, MeshBatch);

#if WITH_EDITOR
						// It's possible the number of batches has changed since the initial frame so validate the BatchIndex
						if (HitProxyIds.IsValidIndex(BatchIndex))
						{
							MeshBatch.BatchHitProxyId = HitProxyIds[BatchIndex];
						}
#endif

						// Apply view mode material overrides
						const int32 MaterialIndex = TrackProxy->Materials.IsValidIndex(BatchInfo.MaterialIndex) ? BatchInfo.MaterialIndex : BatchIndex;
						FMaterialRenderProxy* MaterialProxy = bWireframe ? WireframeMaterialInstance :
															  TrackProxy->Materials.IsValidIndex(MaterialIndex) && TrackProxy->Materials[MaterialIndex] != nullptr ? // extra precaution in case of bad data
															  TrackProxy->Materials[MaterialIndex]->GetRenderProxy() : nullptr;
						MeshBatch.bWireframe = bWireframe;
						MeshBatch.MaterialRenderProxy = MaterialProxy;

						Collector.AddMesh(ViewIndex, MeshBatch);

						INC_DWORD_STAT_BY(STAT_GeometryCacheSceneProxy_TriangleCount, MeshBatch.Elements[0].NumPrimitives);
						INC_DWORD_STAT_BY(STAT_GeometryCacheSceneProxy_MeshBatchCount, 1);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						// Render bounds
						RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
#endif
					}
				}
			}
		}
	}
}

#if RHI_RAYTRACING
void FGeometryCacheSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	if (!CVarRayTracingGeometryCache.GetValueOnRenderThread())
	{
		return;
	}

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

	for (FGeomCacheTrackProxy* TrackProxy : Tracks)
	{
		const FVisibilitySample& VisibilitySample = TrackProxy->GetVisibilitySample(Time, bLooping);
		if (!VisibilitySample.bVisibilityState)
		{
			continue;
		}

		FRayTracingInstance RayTracingInstance;
		RayTracingInstance.Geometry = &TrackProxy->RayTracingGeometry;
		RayTracingInstance.InstanceTransforms.Add(GetLocalToWorld());

		const FGeometryCacheMeshData* MeshData = TrackProxy->bNextFrameMeshDataSelected ? TrackProxy->NextFrameMeshData : TrackProxy->MeshData;
		for (int32 SegmentIndex = 0; SegmentIndex < MeshData->BatchesInfo.Num(); ++SegmentIndex)
		{
			const FGeometryCacheMeshBatchInfo& BatchInfo = MeshData->BatchesInfo[SegmentIndex];
			FMeshBatch MeshBatch;

			FGeometryCacheVertexFactoryUserDataWrapper& UserDataWrapper = Collector.AllocateOneFrameResource<FGeometryCacheVertexFactoryUserDataWrapper>();
			FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
			CreateMeshBatch(Collector.GetRHICommandList(), TrackProxy, BatchInfo, UserDataWrapper, DynamicPrimitiveUniformBuffer, MeshBatch);

			const int32 MaterialIndex = TrackProxy->Materials.IsValidIndex(BatchInfo.MaterialIndex) ? BatchInfo.MaterialIndex : SegmentIndex;
			MeshBatch.MaterialRenderProxy = TrackProxy->Materials[MaterialIndex]->GetRenderProxy();
			MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);
			MeshBatch.SegmentIndex = SegmentIndex;
			MeshBatch.ReverseCulling = false; // RayTracing does not want the transform orientation baked in
			RayTracingInstance.Materials.Add(MeshBatch);
		}

		if (RayTracingInstance.Materials.Num() > 0)
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if ((VisibilityMap & (1 << ViewIndex)) == 0)
				{
					continue;
				}

				Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
			}
		}
	}
}
#endif

FPrimitiveViewRelevance FGeometryCacheSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bDynamicRelevance = true;
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	MaterialRelevance.SetPrimitiveViewRelevance(Result);

	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

	return Result;
}

bool FGeometryCacheSceneProxy::CanBeOccluded() const
{
	return !MaterialRelevance.bDisableDepthTest;
}

bool FGeometryCacheSceneProxy::IsUsingDistanceCullFade() const
{
	return MaterialRelevance.bUsesDistanceCullFade;
}

uint32 FGeometryCacheSceneProxy::GetMemoryFootprint(void) const
{
	return(sizeof(*this) + GetAllocatedSize());
}

uint32 FGeometryCacheSceneProxy::GetAllocatedSize(void) const
{
	return static_cast<uint32>(FPrimitiveSceneProxy::GetAllocatedSize());
}

void FGeometryCacheSceneProxy::UpdateAnimation(FRHICommandListBase& RHICmdList, float NewTime, bool bNewLooping, bool bNewIsPlayingBackwards, float NewPlaybackSpeed, float NewMotionVectorScale, bool bNewInterpolateFrames)
{
	FAnimationUpdateParameters UpdatedValues = 
		{
			NewTime,
			bNewLooping,
			bNewIsPlayingBackwards,
			NewPlaybackSpeed,
			NewMotionVectorScale,
			bNewInterpolateFrames
		};
		UpdateAnimation(RHICmdList, UpdatedValues);
}

void FGeometryCacheSceneProxy::UpdateAnimation(FRHICommandListBase& RHICmdList, const FAnimationUpdateParameters& UpdatedValues)
{
	Time = UpdatedValues.Time;
	bLooping = UpdatedValues.bLooping;
	bIsPlayingBackwards = UpdatedValues.bIsPlayingBackwards;
	PlaybackSpeed = UpdatedValues.PlaybackSpeed;
	MotionVectorScale = UpdatedValues.MotionVectorScale;
	UpdatedFrameNum = GFrameNumber + 1;
	bInterpolateFrames = UpdatedValues.bInterpolateFrames;

	// Always update in render thread regardless of visibility, ray tracing or not
	FrameUpdate(RHICmdList);

	if (IsRayTracingEnabled())
	{
#if RHI_RAYTRACING
		for (FGeomCacheTrackProxy* Section : Tracks)
		{
			if (Section != nullptr)
			{
				const FVisibilitySample& VisibilitySample = Section->GetVisibilitySample(Time, bLooping);
				if (!VisibilitySample.bVisibilityState)
				{
					continue;
				}

				if (!Section->bInitializedRayTracing)
				{
					InitRayTracing(RHICmdList);
				}
			
				const int PositionBufferIndex = Section->CurrentPositionBufferIndex != -1 ? Section->CurrentPositionBufferIndex % 2 : 0;
				const FGeometryCacheMeshData* MeshData = Section->bNextFrameMeshDataSelected ? Section->NextFrameMeshData : Section->MeshData;
				const uint32 IndexBufferNumTriangles = Section->IndexBuffer.NumValidIndices / 3;

				const FRayTracingGeometryInitializer& RayTracingGeometryInitializer = Section->RayTracingGeometry.GetInitializer();
				// Check if a full RaytracingGeometry object needs to be recreated. 
				// Recreate when:
				// - index buffer changes (grew in size)
				// - total primitive count changes
				// - segment count or vertex count changed (change BLAS size)
				bool bRequireRecreate = false;
				bRequireRecreate = bRequireRecreate || RayTracingGeometryInitializer.IndexBuffer != Section->IndexBuffer.IndexBufferRHI;
				bRequireRecreate = bRequireRecreate || RayTracingGeometryInitializer.Segments.Num() != MeshData->BatchesInfo.Num();

				// Validate the max vertex count on all segments 
				if (!bRequireRecreate)
				{
					for (int32 SegmentIndex = 0; SegmentIndex < MeshData->BatchesInfo.Num(); ++SegmentIndex)
					{
						const FGeometryCacheMeshBatchInfo& BatchInfo = MeshData->BatchesInfo[SegmentIndex];
						const FRayTracingGeometrySegment& Segment = RayTracingGeometryInitializer.Segments[SegmentIndex];

						int32 MaxSegmentVertices = Section->PositionBuffers[PositionBufferIndex].GetSizeInBytes() / Segment.VertexBufferStride; // conservative estimate
						bRequireRecreate = bRequireRecreate || Segment.MaxVertices != MaxSegmentVertices;
					}
				}

				uint32 TotalPrimitiveCount = 0;
				TArray<FRayTracingGeometrySegment> Segments;
				Segments.Reserve(MeshData->BatchesInfo.Num());

				for (const FGeometryCacheMeshBatchInfo& BatchInfo : MeshData->BatchesInfo)
				{
					FRayTracingGeometrySegment Segment;
					Segment.FirstPrimitive = BatchInfo.StartIndex / 3;
					Segment.NumPrimitives = BatchInfo.NumTriangles;

					// Ensure that a geometry segment does not access the index buffer out of bounds
					if (!ensureMsgf(Segment.FirstPrimitive + Segment.NumPrimitives <= IndexBufferNumTriangles,
						TEXT("Ray tracing geometry index buffer is smaller than what's required by FGeometryCacheMeshBatchInfo. ")
						TEXT("Segment.FirstPrimitive=%d Segment.NumPrimitives=%d RequiredIndexBufferTriangles=%d IndexBufferNumTriangles=%d"),
						Segment.FirstPrimitive, Segment.NumPrimitives, Segment.FirstPrimitive+Segment.NumPrimitives, IndexBufferNumTriangles))
					{
						Segment.NumPrimitives = IndexBufferNumTriangles - FMath::Min<uint32>(Segment.FirstPrimitive, IndexBufferNumTriangles);
					}

					Segment.VertexBuffer = Section->PositionBuffers[PositionBufferIndex].VertexBufferRHI;
					Segment.MaxVertices = Section->PositionBuffers[PositionBufferIndex].GetSizeInBytes() / Segment.VertexBufferStride; // conservative estimate

					Segments.Add(Segment);
					TotalPrimitiveCount += Segment.NumPrimitives;
				}

				bRequireRecreate = bRequireRecreate || RayTracingGeometryInitializer.TotalPrimitiveCount != TotalPrimitiveCount;

				if (bRequireRecreate)
				{
					Section->RayTracingGeometry.ReleaseRHI();

					{
						FRayTracingGeometryInitializer NewRayTracingGeometryInitializer = RayTracingGeometryInitializer;
						NewRayTracingGeometryInitializer.IndexBuffer = Section->IndexBuffer.IndexBufferRHI;
						NewRayTracingGeometryInitializer.TotalPrimitiveCount = TotalPrimitiveCount;
						NewRayTracingGeometryInitializer.Segments = MoveTemp(Segments);

						Section->RayTracingGeometry.SetInitializer(MoveTemp(NewRayTracingGeometryInitializer));
					}

					Section->RayTracingGeometry.InitRHI(RHICmdList);
				}
				else if(Section->RayTracingGeometry.IsValid() && !Section->RayTracingGeometry.IsEvicted())
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// TODO: Shouldn't directly modify FRayTracingGeometry initializer
					Section->RayTracingGeometry.Initializer.Segments = MoveTemp(Segments);

					// Request full build on same geometry because data might have changed to much for update call?
					FRayTracingGeometryBuildParams BuildParams;
					BuildParams.Geometry = Section->RayTracingGeometry.GetRHI();
					BuildParams.BuildMode = EAccelerationStructureBuildMode::Build;
					BuildParams.Segments = Section->RayTracingGeometry.Initializer.Segments;
					FRHIComputeCommandList::Get(RHICmdList).BuildAccelerationStructures(MakeArrayView(&BuildParams, 1));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}
#endif
	}
}

void FGeometryCacheSceneProxy::FrameUpdate(FRHICommandListBase& RHICmdList) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSceneProxy::FrameUpdate);

	for (FGeomCacheTrackProxy* TrackProxy : Tracks)
	{
		// Render out stored TrackProxy's
		if (TrackProxy != nullptr)
		{
			const FVisibilitySample& VisibilitySample = TrackProxy->GetVisibilitySample(Time, bLooping);
			if (!VisibilitySample.bVisibilityState)
			{
				continue;
			}

			// Figure out which frame(s) we need to decode
			int32 FrameIndex;
			int32 NextFrameIndex;
			float InterpolationFactor;
			TrackProxy->SubframeInterpolationFactor = 1.0f;
			TrackProxy->PreviousFrameIndex = TrackProxy->FrameIndex;
			TrackProxy->PreviousInterpolationFactor = TrackProxy->InterpolationFactor;
			TrackProxy->FindSampleIndexesFromTime(Time, bLooping, bIsPlayingBackwards, FrameIndex, NextFrameIndex, InterpolationFactor);
			bool bDecodedAnything = false; // Did anything new get decoded this frame
			bool bSeeked = false; // Is this frame a seek and thus the previous rendered frame's data invalid
			bool bDecoderError = false; // If we have a decoder error we don't interpolate and we don't update the vertex buffers
										// so essentially we just keep the last valid frame...

			bool bFrameIndicesChanged = false;
			const bool bDifferentRoundedInterpolationFactor = FMath::RoundToInt(InterpolationFactor) != FMath::RoundToInt(TrackProxy->InterpolationFactor);
			const bool bDifferentInterpolationFactor = !FMath::IsNearlyEqual(InterpolationFactor, TrackProxy->InterpolationFactor);
			TrackProxy->InterpolationFactor = InterpolationFactor;

			// Compare this against the frames we got and keep some/all/none of them
			// This will work across frames but also within a frame if the mesh is in several views
			if (TrackProxy->FrameIndex != FrameIndex || TrackProxy->NextFrameIndex != NextFrameIndex)
			{
				// Normal case the next frame is the new current frame
				if (TrackProxy->NextFrameIndex == FrameIndex)
				{
					// Cycle the current and next frame double buffer
					FGeometryCacheMeshData* OldFrameMesh = TrackProxy->MeshData;
					TrackProxy->MeshData = TrackProxy->NextFrameMeshData;
					TrackProxy->NextFrameMeshData = OldFrameMesh;

					int32 OldFrameIndex = TrackProxy->FrameIndex;
					TrackProxy->FrameIndex = TrackProxy->NextFrameIndex;
					TrackProxy->NextFrameIndex = OldFrameIndex;

					// Decode the new next frame
					if (TrackProxy->GetMeshData(NextFrameIndex, *TrackProxy->NextFrameMeshData)) //-V1051
					{
						bDecodedAnything = true;
						// Only register this if we actually successfully decoded
						TrackProxy->NextFrameIndex = NextFrameIndex;
					}
					else
					{
						// Mark the frame as corrupted
						TrackProxy->NextFrameIndex = -1;
						bDecoderError = true;
					}
				}
				// Probably a seek or the mesh hasn't been visible in a while decode two frames
				else
				{
					if (TrackProxy->GetMeshData(FrameIndex, *TrackProxy->MeshData))
					{
						TrackProxy->NextFrameMeshData->Indices = TrackProxy->MeshData->Indices;
						if (TrackProxy->GetMeshData(NextFrameIndex, *TrackProxy->NextFrameMeshData))
						{
							TrackProxy->FrameIndex = FrameIndex;
							TrackProxy->NextFrameIndex = NextFrameIndex;
							bSeeked = true;
							bDecodedAnything = true;
						}
						else
						{
							// The first frame decoded fine but the second didn't 
							// we need to specially handle this
							TrackProxy->NextFrameIndex = -1;
							bDecoderError = true;
						}
					}
					else
					{
						TrackProxy->FrameIndex = -1;
						TrackProxy->PreviousFrameIndex = -1;
						bDecoderError = true;
					}
				}

				bFrameIndicesChanged = true;
			}

			// Check if we can interpolate between the two frames we have available
			const bool bCanInterpolate = TrackProxy->IsTopologyCompatible(TrackProxy->FrameIndex, TrackProxy->NextFrameIndex) && bInterpolateFrames;

			// Check if we have explicit motion vectors
			const bool bHasMotionVectors = (
				TrackProxy->MeshData->VertexInfo.bHasMotionVectors &&
				TrackProxy->NextFrameMeshData->VertexInfo.bHasMotionVectors &&
				TrackProxy->MeshData->Positions.Num() == TrackProxy->MeshData->MotionVectors.Num())
				&& (TrackProxy->NextFrameMeshData->Positions.Num() == TrackProxy->NextFrameMeshData->MotionVectors.Num());

			// Can we interpolate the vertex data?
			if (bCanInterpolate && (bDifferentInterpolationFactor || bFrameIndicesChanged) && !bDecoderError && CVarInterpolateFrames.GetValueOnRenderThread() != 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSceneProxy::FrameUpdate::Interpolate);

				TrackProxy->bNextFrameMeshDataSelected = false;

				SCOPE_CYCLE_COUNTER(STAT_InterpolateFrames);
				// Interpolate if the time has changed.
				// note: This is a bit precarious as this code is called multiple times per frame. This ensures
				// we only interpolate once (which is a nice optimization) but more importantly that we only
				// bump the CurrentPositionBufferIndex once per frame. This ensures that last frame's position
				// buffer is not overwritten.
				// If motion blur suddenly seems to stop working while it should be working it may be that the
				// CurrentPositionBufferIndex gets inadvertently bumped twice per frame essentially using the same
				// data for current and previous during rendering.

				const int32 NumVerts = TrackProxy->MeshData->Positions.Num();

				if (NumVerts == 0)
				{
					return;
				}
				else if (!TrackProxy->bResourcesInitialized)
				{
					TrackProxy->InitRenderResources(RHICmdList, NumVerts, TrackProxy->MeshData->Indices.Num());
				}

				Scratch.Prepare(NumVerts, bHasMotionVectors);

				const float OneMinusInterp = 1.0f - InterpolationFactor;
				const int32 InterpFixed = (int32)(InterpolationFactor * 255.0f);
				const int32 OneMinusInterpFixed = 255 - InterpFixed;
				const VectorRegister4Float WeightA = VectorSetFloat1( OneMinusInterp );
				const VectorRegister4Float WeightB = VectorSetFloat1( InterpolationFactor );
				const VectorRegister4Float Half = VectorSetFloat1( 0.5f );

				#define VALIDATE 0
				{

					check(TrackProxy->MeshData->Positions.Num() >= NumVerts);
					check(TrackProxy->NextFrameMeshData->Positions.Num() >= NumVerts);
					check(Scratch.InterpolatedPositions.Num() >= NumVerts);
					const FVector3f* PositionAPtr = TrackProxy->MeshData->Positions.GetData();
					const FVector3f* PositionBPtr = TrackProxy->NextFrameMeshData->Positions.GetData();
					FVector3f* InterpolatedPositionsPtr = Scratch.InterpolatedPositions.GetData();

					SCOPE_CYCLE_COUNTER(STAT_InterpolateFramesSIMDPositions);

					// Unroll 4 times so we can do 4 wide SIMD
#if INTEL_ISPC
					if (GGeometryCacheSceneProxyUseIspc)
					{
						ispc::InterpolatePositions(
							(ispc::FVector3f*)PositionAPtr,
							(ispc::FVector3f*)PositionBPtr,
							(ispc::FVector3f*)InterpolatedPositionsPtr,
							NumVerts,
							OneMinusInterp,
							InterpolationFactor);
					}
					else
#endif
					{
						const FVector4f* PositionAPtr4 = (const FVector4f*)PositionAPtr;
						const FVector4f* PositionBPtr4 = (const FVector4f*)PositionBPtr;
						FVector4f* InterpolatedPositionsPtr4 = (FVector4f*)InterpolatedPositionsPtr;

						int32 Index = 0;
						for (; Index + 3 < NumVerts; Index += 4)
						{
							VectorRegister4Float Pos0xyz_Pos1x = VectorMultiplyAdd(VectorLoad(PositionAPtr4 + 0), WeightA, VectorMultiply(VectorLoad(PositionBPtr4 + 0), WeightB));
							VectorRegister4Float Pos1yz_Pos2xy = VectorMultiplyAdd(VectorLoad(PositionAPtr4 + 1), WeightA, VectorMultiply(VectorLoad(PositionBPtr4 + 1), WeightB));
							VectorRegister4Float Pos2z_Pos3xyz = VectorMultiplyAdd(VectorLoad(PositionAPtr4 + 2), WeightA, VectorMultiply(VectorLoad(PositionBPtr4 + 2), WeightB));
							VectorStore(Pos0xyz_Pos1x, InterpolatedPositionsPtr4 + 0);
							VectorStore(Pos1yz_Pos2xy, InterpolatedPositionsPtr4 + 1);
							VectorStore(Pos2z_Pos3xyz, InterpolatedPositionsPtr4 + 2);
							PositionAPtr4 += 3;
							PositionBPtr4 += 3;
							InterpolatedPositionsPtr4 += 3;
						}

						for (; Index < NumVerts; Index++)
						{
							InterpolatedPositionsPtr[Index] = PositionAPtr[Index] * OneMinusInterp + PositionBPtr[Index] * InterpolationFactor;
						}
					}
#if VALIDATE
					for (int32 Index = 0; Index < NumVerts; ++Index)
					{
						FVector3f Result = PositionAPtr[Index] * OneMinusInterp + PositionBPtr[Index] * InterpolationFactor;
						check(FMath::Abs(InterpolatedPositionsPtr[Index].X - Result.X) < 0.01f);
						check(FMath::Abs(InterpolatedPositionsPtr[Index].Y - Result.Y) < 0.01f);
						check(FMath::Abs(InterpolatedPositionsPtr[Index].Z - Result.Z) < 0.01f);
					}
#endif
				}
				
				{
					check(TrackProxy->MeshData->TangentsX.Num() >= NumVerts);
					check(TrackProxy->NextFrameMeshData->TangentsX.Num() >= NumVerts);
					check(TrackProxy->MeshData->TangentsZ.Num() >= NumVerts);
					check(TrackProxy->NextFrameMeshData->TangentsZ.Num() >= NumVerts);
					check(Scratch.InterpolatedTangentX.Num() >= NumVerts);
					check(Scratch.InterpolatedTangentZ.Num() >= NumVerts);
					const FPackedNormal* TangentXAPtr = TrackProxy->MeshData->TangentsX.GetData();
					const FPackedNormal* TangentXBPtr = TrackProxy->NextFrameMeshData->TangentsX.GetData();
					const FPackedNormal* TangentZAPtr = TrackProxy->MeshData->TangentsZ.GetData();
					const FPackedNormal* TangentZBPtr = TrackProxy->NextFrameMeshData->TangentsZ.GetData();
					FPackedNormal* InterpolatedTangentXPtr = Scratch.InterpolatedTangentX.GetData();
					FPackedNormal* InterpolatedTangentZPtr = Scratch.InterpolatedTangentZ.GetData();

					SCOPE_CYCLE_COUNTER(STAT_InterpolateFramesSIMDTangents);

#if INTEL_ISPC
					if (GGeometryCacheSceneProxyUseIspc)
					{
						ispc::InterpolateTangents(
							(uint8*)TangentXAPtr,
							(uint8*)TangentXBPtr,
							(uint8*)TangentZAPtr,
							(uint8*)TangentZBPtr,
							(uint8*)InterpolatedTangentXPtr,
							(uint8*)InterpolatedTangentZPtr,
							NumVerts * sizeof(uint32),
							OneMinusInterp,
							InterpolationFactor);
					}
					else
#endif
					{
						const uint32 SignMask = 0x80808080u;
						for (int32 Index = 0; Index < NumVerts; ++Index)
						{
							// VectorLoadSignedByte4 on all inputs is significantly more expensive than VectorLoadByte4, so lets just use unsigned.
							// Interpolating signed values as unsigned is not correct, but if we flip the signs first it is!
							// Flipping the sign maps the signed range [-128, 127] to the unsigned range [0, 255]
							// Unsigned value with flip			Signed value
							// 0								-128
							// 1								-127
							// ..								..
							// 127								-1
							// 128								0
							// 129								1
							// 255								127

							uint32 TangentXA = TangentXAPtr[Index].Vector.Packed ^ SignMask;
							uint32 TangentXB = TangentXBPtr[Index].Vector.Packed ^ SignMask;
							VectorRegister4Float InterpolatedTangentX = VectorMultiplyAdd(VectorLoadByte4(&TangentXA), WeightA,
								VectorMultiplyAdd(VectorLoadByte4(&TangentXB), WeightB, Half));	// +0.5f so truncation becomes round to nearest.
							uint32 PackedInterpolatedTangentX;
							VectorStoreByte4(InterpolatedTangentX, &PackedInterpolatedTangentX);
							InterpolatedTangentXPtr[Index].Vector.Packed = PackedInterpolatedTangentX ^ SignMask;	// Convert back to signed

							uint32 TangentZA = TangentZAPtr[Index].Vector.Packed ^ SignMask;
							uint32 TangentZB = TangentZBPtr[Index].Vector.Packed ^ SignMask;
							VectorRegister4Float InterpolatedTangentZ = VectorMultiplyAdd(VectorLoadByte4(&TangentZA), WeightA,
								VectorMultiplyAdd(VectorLoadByte4(&TangentZB), WeightB, Half));	// +0.5f so truncation becomes round to nearest.
							uint32 PackedInterpolatedTangentZ;
							VectorStoreByte4(InterpolatedTangentZ, &PackedInterpolatedTangentZ);
							InterpolatedTangentZPtr[Index].Vector.Packed = PackedInterpolatedTangentZ ^ SignMask;	// Convert back to signed
						}
						VectorResetFloatRegisters();	//TODO: is this actually needed on any platform?

#if VALIDATE
						for (int32 Index = 0; Index < NumVerts; ++Index)
						{
							FPackedNormal ResultX = InterpolatePackedNormal(TangentXAPtr[Index], TangentXBPtr[Index], InterpFixed, OneMinusInterpFixed);
							FPackedNormal ResultZ = InterpolatePackedNormal(TangentZAPtr[Index], TangentZBPtr[Index], InterpFixed, OneMinusInterpFixed);
							check(FMath::Abs(InterpolatedTangentXPtr[Index].Vector.X - ResultX.Vector.X) <= 2);
							check(FMath::Abs(InterpolatedTangentXPtr[Index].Vector.Y - ResultX.Vector.Y) <= 2);
							check(FMath::Abs(InterpolatedTangentXPtr[Index].Vector.Z - ResultX.Vector.Z) <= 2);
							check(FMath::Abs(InterpolatedTangentXPtr[Index].Vector.W - ResultX.Vector.W) <= 2);

							check(FMath::Abs(InterpolatedTangentZPtr[Index].Vector.X - ResultZ.Vector.X) <= 2);
							check(FMath::Abs(InterpolatedTangentZPtr[Index].Vector.Y - ResultZ.Vector.Y) <= 2);
							check(FMath::Abs(InterpolatedTangentZPtr[Index].Vector.Z - ResultZ.Vector.Z) <= 2);
							check(FMath::Abs(InterpolatedTangentZPtr[Index].Vector.W - ResultZ.Vector.W) <= 2);
						}
#endif
					}
				}

				if (TrackProxy->MeshData->VertexInfo.bHasColor0)
				{
					check(TrackProxy->MeshData->Colors.Num() >= NumVerts);
					check(TrackProxy->NextFrameMeshData->Colors.Num() >= NumVerts);
					check(Scratch.InterpolatedColors.Num() >= NumVerts);
					const FColor* ColorAPtr = TrackProxy->MeshData->Colors.GetData();
					const FColor* ColorBPtr = TrackProxy->NextFrameMeshData->Colors.GetData();
					FColor* InterpolatedColorsPtr = Scratch.InterpolatedColors.GetData();

					SCOPE_CYCLE_COUNTER(STAT_InterpolateFramesSIMDColors);
#if INTEL_ISPC
					if (GGeometryCacheSceneProxyUseIspc)
					{
						ispc::InterpolateColors(
							(uint8*)ColorAPtr,
							(uint8*)ColorBPtr,
							(uint8*)InterpolatedColorsPtr,
							NumVerts * sizeof(uint32),
							OneMinusInterp,
							InterpolationFactor);
					}
					else
#endif
					{
						for (int32 Index = 0; Index < NumVerts; ++Index)
						{
							VectorRegister4Float InterpolatedColor = VectorMultiplyAdd(VectorLoadByte4(&ColorAPtr[Index]), WeightA,
								VectorMultiplyAdd(VectorLoadByte4(&ColorBPtr[Index]), WeightB, Half));	// +0.5f so truncation becomes round to nearest.
							VectorStoreByte4(InterpolatedColor, &InterpolatedColorsPtr[Index]);
						}
					}
#if VALIDATE
					for(int32 Index = 0; Index < NumVerts; ++Index)
					{
						const FColor& ColorA = ColorAPtr[Index];
						const FColor& ColorB = ColorBPtr[Index];
						FColor Result = InterpolatePackedColor(ColorA, ColorB, InterpFixed, OneMinusInterpFixed);
						check(FMath::Abs(InterpolatedColorsPtr[Index].R - Result.R) <= 1);
						check(FMath::Abs(InterpolatedColorsPtr[Index].G - Result.G) <= 1);
						check(FMath::Abs(InterpolatedColorsPtr[Index].B - Result.B) <= 1);
						check(FMath::Abs(InterpolatedColorsPtr[Index].A - Result.A) <= 1);
					}
#endif
				}

				if (TrackProxy->MeshData->VertexInfo.bHasUV0)
				{
					check(TrackProxy->MeshData->TextureCoordinates.Num() >= NumVerts);
					check(TrackProxy->NextFrameMeshData->TextureCoordinates.Num() >= NumVerts);
					check(Scratch.InterpolatedUVs.Num() >= NumVerts);
					const FVector2f* UVAPtr = TrackProxy->MeshData->TextureCoordinates.GetData();
					const FVector2f* UVBPtr = TrackProxy->NextFrameMeshData->TextureCoordinates.GetData();
					FVector2f* InterpolatedUVsPtr = Scratch.InterpolatedUVs.GetData();

					SCOPE_CYCLE_COUNTER(STAT_InterpolateFramesSIMDUVs);

#if INTEL_ISPC
					if (GGeometryCacheSceneProxyUseIspc)
					{
						ispc::InterpolateUVs(
							(ispc::FVector2f*)UVAPtr,
							(ispc::FVector2f*)UVBPtr,
							(ispc::FVector2f*)InterpolatedUVsPtr,
							NumVerts,
							OneMinusInterp,
							InterpolationFactor);
					}
					else
#endif
					{
						// Unroll 2x so we can use 4 wide ops. OOP will hopefully take care of the rest.
						{
							int32 Index = 0;
							for (; Index + 1 < NumVerts; Index += 2)
							{
								VectorRegister4Float InterpolatedUVx2 = VectorMultiplyAdd(VectorLoad(&UVAPtr[Index].X),
									WeightA,
									VectorMultiply(VectorLoad(&UVBPtr[Index].X), WeightB));
								VectorStore(InterpolatedUVx2, &(InterpolatedUVsPtr[Index].X));
							}

							if (Index < NumVerts)
							{
								InterpolatedUVsPtr[Index] = UVAPtr[Index] * OneMinusInterp + UVBPtr[Index] * InterpolationFactor;
							}
						}

#if VALIDATE
						for (int32 Index = 0; Index < NumVerts; ++Index)
						{
							FVector2D Result = UVAPtr[Index] * OneMinusInterp + UVBPtr[Index] * InterpolationFactor;
							check(FMath::Abs(InterpolatedUVsPtr[Index].X - Result.X) < 0.01f);
							check(FMath::Abs(InterpolatedUVsPtr[Index].Y - Result.Y) < 0.01f);
						}
#endif
					}
				}

				if (bHasMotionVectors)
				{
					check(TrackProxy->MeshData->MotionVectors.Num() >= NumVerts);
					check(TrackProxy->NextFrameMeshData->MotionVectors.Num() >= NumVerts);
					check(Scratch.InterpolatedMotionVectors.Num() >= NumVerts);
					const FVector3f* MotionVectorsAPtr = TrackProxy->MeshData->MotionVectors.GetData();
					const FVector3f* MotionVectorsBPtr = TrackProxy->NextFrameMeshData->MotionVectors.GetData();
					FVector3f* InterpolatedMotionVectorsPtr = Scratch.InterpolatedMotionVectors.GetData();

					// The subframe interpolation factor is the multiplier that should be applied to the motion vectors to account for subframe sampling
					// It represents the delta interpolation factor between each sub-frame (due to temporal subsampling)
					// but we don't want to affect the motion vectors when sampling at multiples of frame so it's clamped to 1
					float DeltaInterpolationFactor = InterpolationFactor - TrackProxy->PreviousInterpolationFactor;
					DeltaInterpolationFactor += static_cast<float>(TrackProxy->FrameIndex - TrackProxy->PreviousFrameIndex);
					DeltaInterpolationFactor = FMath::Clamp(FMath::Abs(DeltaInterpolationFactor), 0.0f, 1.0f); // the Abs accounts for playing backwards
					TrackProxy->SubframeInterpolationFactor = FMath::IsNearlyEqual(DeltaInterpolationFactor, 1.0f, KINDA_SMALL_NUMBER) ? 1.0f : DeltaInterpolationFactor;

					SCOPE_CYCLE_COUNTER(STAT_InterpolateFramesSIMDMotionVectors);

#if INTEL_ISPC
					if (GGeometryCacheSceneProxyUseIspc)
					{
						ispc::InterpolateMotionVectors(
							(ispc::FVector3f*)MotionVectorsAPtr,
							(ispc::FVector3f*)MotionVectorsBPtr,
							(ispc::FVector3f*)InterpolatedMotionVectorsPtr,
							NumVerts,
							OneMinusInterp,
							InterpolationFactor,
							MotionVectorScale);
					}
					else
#endif

					// Unroll 4 times so we can do 4 wide SIMD
					{
						const VectorRegister4Float ScaledWeightA = VectorSetFloat1(OneMinusInterp * MotionVectorScale);
						const VectorRegister4Float ScaledWeightB = VectorSetFloat1(InterpolationFactor * MotionVectorScale);
						const FVector4f* MotionVectorsAPtr4 = (const FVector4f*)MotionVectorsAPtr;
						const FVector4f* MotionVectorsBPtr4 = (const FVector4f*)MotionVectorsBPtr;
						FVector4f* InterpolatedMotionVectorsPtr4 = (FVector4f*)InterpolatedMotionVectorsPtr;

						int32 Index = 0;
						for (; Index + 3 < NumVerts; Index += 4)
						{
							VectorRegister4Float MotionVector0xyz_MotionVector1x = VectorMultiplyAdd(VectorLoad(MotionVectorsAPtr4 + 0), ScaledWeightA, VectorMultiply(VectorLoad(MotionVectorsBPtr4 + 0), ScaledWeightB));
							VectorRegister4Float MotionVector1yz_MotionVector2xy = VectorMultiplyAdd(VectorLoad(MotionVectorsAPtr4 + 1), ScaledWeightA, VectorMultiply(VectorLoad(MotionVectorsBPtr4 + 1), ScaledWeightB));
							VectorRegister4Float MotionVector2z_MotionVector3xyz = VectorMultiplyAdd(VectorLoad(MotionVectorsAPtr4 + 2), ScaledWeightA, VectorMultiply(VectorLoad(MotionVectorsBPtr4 + 2), ScaledWeightB));
							VectorStore(MotionVector0xyz_MotionVector1x, InterpolatedMotionVectorsPtr4 + 0);
							VectorStore(MotionVector1yz_MotionVector2xy, InterpolatedMotionVectorsPtr4 + 1);
							VectorStore(MotionVector2z_MotionVector3xyz, InterpolatedMotionVectorsPtr4 + 2);
							MotionVectorsAPtr4 += 3;
							MotionVectorsBPtr4 += 3;
							InterpolatedMotionVectorsPtr4 += 3;
						}

						for (; Index < NumVerts; Index++)
						{
							InterpolatedMotionVectorsPtr[Index] = (MotionVectorsAPtr[Index] * OneMinusInterp + MotionVectorsBPtr[Index] * InterpolationFactor) * MotionVectorScale;
						}
					}
#if VALIDATE
					for (int32 Index = 0; Index < NumVerts; ++Index)
					{
						FVector3f Result = (MotionVectorsAPtr[Index] * OneMinusInterp + MotionVectorsBPtr[Index] * InterpolationFactor) * MotionVectorScale;
						check(FMath::Abs(InterpolatedMotionVectorsPtr[Index].X - Result.X) < 0.01f);
						check(FMath::Abs(InterpolatedMotionVectorsPtr[Index].Y - Result.Y) < 0.01f);
						check(FMath::Abs(InterpolatedMotionVectorsPtr[Index].Z - Result.Z) < 0.01f);
					}
#endif
				}

#undef VALIDATE

				// Upload other non-motionblurred data
				if (!TrackProxy->MeshData->VertexInfo.bConstantIndices)
					TrackProxy->IndexBuffer.Update(RHICmdList, TrackProxy->MeshData->Indices);

				if (TrackProxy->MeshData->VertexInfo.bHasTangentX)
					TrackProxy->TangentXBuffer.Update(RHICmdList, Scratch.InterpolatedTangentX);
				if (TrackProxy->MeshData->VertexInfo.bHasTangentZ)
					TrackProxy->TangentZBuffer.Update(RHICmdList, Scratch.InterpolatedTangentZ);

				if (TrackProxy->MeshData->VertexInfo.bHasUV0)
					TrackProxy->TextureCoordinatesBuffer.Update(RHICmdList, Scratch.InterpolatedUVs);

				if (TrackProxy->MeshData->VertexInfo.bHasColor0)
					TrackProxy->ColorBuffer.Update(RHICmdList, Scratch.InterpolatedColors);

				bool bIsCompatibleWithCachedFrame = TrackProxy->IsTopologyCompatible(
					TrackProxy->PositionBufferFrameIndices[TrackProxy->CurrentPositionBufferIndex % 2],
					TrackProxy->FrameIndex);

				if (!bHasMotionVectors)
				{
					// Initialize both buffers the first frame
					if (TrackProxy->CurrentPositionBufferIndex == -1 || !bIsCompatibleWithCachedFrame)
					{
						TrackProxy->PositionBuffers[0].Update(RHICmdList, Scratch.InterpolatedPositions);
						TrackProxy->PositionBuffers[1].Update(RHICmdList, Scratch.InterpolatedPositions);
						TrackProxy->CurrentPositionBufferIndex = 0;
						TrackProxy->PositionBufferFrameTimes[0] = Time;
						TrackProxy->PositionBufferFrameTimes[1] = Time;
						// We need to keep a frame index in order to ensure topology consistency. As we can interpolate 
						// FrameIndex and NextFrameIndex are certainly topo-compatible so it doesn't really matter which 
						// one we keep here. But wee keep NextFrameIndex as that is most useful to validate against
						// the frame coming up
						TrackProxy->PositionBufferFrameIndices[0] = TrackProxy->NextFrameIndex;
						TrackProxy->PositionBufferFrameIndices[1] = TrackProxy->NextFrameIndex;
					}
					else
					{
						TrackProxy->CurrentPositionBufferIndex++;
						TrackProxy->PositionBuffers[TrackProxy->CurrentPositionBufferIndex % 2].Update(RHICmdList, Scratch.InterpolatedPositions);
						TrackProxy->PositionBufferFrameTimes[TrackProxy->CurrentPositionBufferIndex % 2] = Time;
						TrackProxy->PositionBufferFrameIndices[TrackProxy->CurrentPositionBufferIndex % 2] = TrackProxy->NextFrameIndex;
					}
				}
				else
				{
					TrackProxy->CurrentPositionBufferIndex = 0;
					TrackProxy->PositionBuffers[0].Update(RHICmdList, Scratch.InterpolatedPositions);
					TrackProxy->PositionBuffers[1].Update(RHICmdList, Scratch.InterpolatedMotionVectors);
					TrackProxy->PositionBufferFrameIndices[0] = TrackProxy->FrameIndex;
					TrackProxy->PositionBufferFrameIndices[1] = -1;
					TrackProxy->PositionBufferFrameTimes[0] = Time;
					TrackProxy->PositionBufferFrameTimes[1] = Time;
				}
			}
			else
			{
				// We just don't interpolate between frames if we got GPU to burn we could someday render twice and stipple fade between it :-D like with lods
				TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheSceneProxy::FrameUpdate::NonInterpolate);

				// Only bother uploading if anything changed or when the we failed to decode anything make sure update the gpu buffers regardless
				if (bFrameIndicesChanged || bDifferentRoundedInterpolationFactor || (bDifferentInterpolationFactor && bExtrapolateFrames) || bDecodedAnything || bDecoderError)
				{
					const bool bNextFrame = !!FMath::RoundToInt(InterpolationFactor) && TrackProxy->NextFrameMeshData->Positions.Num() > 0 && (TrackProxy->NextFrameIndex != -1); // use next frame only if it's valid
					const uint32 FrameIndexToUse = bNextFrame ? TrackProxy->NextFrameIndex : TrackProxy->FrameIndex;
					FGeometryCacheMeshData* MeshDataToUse = bNextFrame ? TrackProxy->NextFrameMeshData : TrackProxy->MeshData;

					if (MeshDataToUse->Positions.Num() == 0)
					{
						return;
					}
					else if (!TrackProxy->bResourcesInitialized)
					{
						TrackProxy->InitRenderResources(RHICmdList, MeshDataToUse->Positions.Num(), MeshDataToUse->Indices.Num());
					}

					TrackProxy->bNextFrameMeshDataSelected = bNextFrame;

					const int32 NumVertices = MeshDataToUse->Positions.Num();

					if (MeshDataToUse->VertexInfo.bHasTangentX)
						TrackProxy->TangentXBuffer.Update(RHICmdList, MeshDataToUse->TangentsX);
					if (MeshDataToUse->VertexInfo.bHasTangentZ)
						TrackProxy->TangentZBuffer.Update(RHICmdList, MeshDataToUse->TangentsZ);

					if (!MeshDataToUse->VertexInfo.bConstantIndices)
						TrackProxy->IndexBuffer.Update(RHICmdList, MeshDataToUse->Indices);

					if (MeshDataToUse->VertexInfo.bHasUV0)
						TrackProxy->TextureCoordinatesBuffer.Update(RHICmdList, MeshDataToUse->TextureCoordinates);

					if (MeshDataToUse->VertexInfo.bHasColor0)
						TrackProxy->ColorBuffer.Update(RHICmdList, MeshDataToUse->Colors);

					const bool bIsCompatibleWithCachedFrame = TrackProxy->IsTopologyCompatible(
						TrackProxy->PositionBufferFrameIndices[TrackProxy->CurrentPositionBufferIndex % 2],
						FrameIndexToUse);

					if (!bHasMotionVectors)
					{
						// Initialize both buffers the first frame or when topology changed as we can't render
						// with a previous buffer referencing a buffer from another topology
						if (TrackProxy->CurrentPositionBufferIndex == -1 || !bIsCompatibleWithCachedFrame || bSeeked)
						{
							TrackProxy->PositionBuffers[0].Update(RHICmdList, MeshDataToUse->Positions);
							TrackProxy->PositionBuffers[1].Update(RHICmdList, MeshDataToUse->Positions);
							TrackProxy->CurrentPositionBufferIndex = 0;
							TrackProxy->PositionBufferFrameIndices[0] = FrameIndexToUse;
							TrackProxy->PositionBufferFrameIndices[1] = FrameIndexToUse;
						}
						// We still use the previous frame's buffer as a motion blur previous position. As interpolation is switched
						// off the actual time of this previous frame depends on the geometry cache framerate and playback speed
						// so the motion blur vectors may not really be anything relevant. Do we want to just disable motion blur? 
						// But as an optimization skipping interpolation when the cache fps is near to the actual game fps this is obviously nice...
						else
						{
							TrackProxy->CurrentPositionBufferIndex++;
							TrackProxy->PositionBuffers[TrackProxy->CurrentPositionBufferIndex % 2].Update(RHICmdList, MeshDataToUse->Positions);
							TrackProxy->PositionBufferFrameIndices[TrackProxy->CurrentPositionBufferIndex % 2] = FrameIndexToUse;
						}
					}
					else
					{
						TArray<FVector3f> ScaledMotionVectors;
						const bool bScaleMotionVectors = !FMath::IsNearlyEqual(MotionVectorScale, 1.0f);
						if (bScaleMotionVectors)
						{
							float InMotionVectorScale = MotionVectorScale;
							ScaledMotionVectors.SetNum(MeshDataToUse->Positions.Num());
							ParallelFor(MeshDataToUse->Positions.Num(), [InMotionVectorScale, &ScaledMotionVectors, &MeshDataToUse](int32 Index)
								{
									ScaledMotionVectors[Index] = MeshDataToUse->MotionVectors[Index] * InMotionVectorScale;
								});
						}

						const TArray<FVector3f>& MotionVectors = bScaleMotionVectors ? ScaledMotionVectors : MeshDataToUse->MotionVectors;

						TArray<FVector3f> ExtrapolatedPositions;
						if (bExtrapolateFrames)
						{
							ExtrapolatedPositions.SetNum(MeshDataToUse->Positions.Num());
							// Shift the interpolation factor so that it varies between -0.5 and 0.5 around the frame
							const float ShiftedInterpolationFactor = bNextFrame ? InterpolationFactor - 1.0f : InterpolationFactor;
							ParallelFor(MeshDataToUse->Positions.Num(), [ShiftedInterpolationFactor, &MeshDataToUse, &ExtrapolatedPositions, &MotionVectors](int32 Index)
								{
									ExtrapolatedPositions[Index] = MeshDataToUse->Positions[Index] - ShiftedInterpolationFactor * MotionVectors[Index];
								});
						}

						float DeltaInterpolationFactor = InterpolationFactor - TrackProxy->PreviousInterpolationFactor;
						DeltaInterpolationFactor += static_cast<float>(TrackProxy->FrameIndex - TrackProxy->PreviousFrameIndex);
						DeltaInterpolationFactor = FMath::Clamp(FMath::Abs(DeltaInterpolationFactor), 0.0f, 1.0f);
						TrackProxy->SubframeInterpolationFactor = FMath::IsNearlyEqual(DeltaInterpolationFactor, 1.0f, KINDA_SMALL_NUMBER) ? 1.0f : DeltaInterpolationFactor;

						TrackProxy->CurrentPositionBufferIndex = 0;
						TrackProxy->PositionBuffers[0].Update(RHICmdList, bExtrapolateFrames ? ExtrapolatedPositions : MeshDataToUse->Positions);
						TrackProxy->PositionBuffers[1].Update(RHICmdList, MotionVectors);
						TrackProxy->PositionBufferFrameIndices[0] = FrameIndexToUse;
						TrackProxy->PositionBufferFrameIndices[1] = -1;
						TrackProxy->PositionBufferFrameTimes[0] = Time;
						TrackProxy->PositionBufferFrameTimes[1] = Time;
					}
				}
			}

#if 0
			bool bOffloadUpdate = CVarOffloadUpdate.GetValueOnRenderThread() != 0;
			if (TrackProxy->SampleIndex != TrackProxy->UploadedSampleIndex)
			{
				TrackProxy->UploadedSampleIndex = TrackProxy->SampleIndex;

				if (bOffloadUpdate)
				{
					check(false);
					// Only update the size on this thread
					TrackProxy->IndexBuffer.UpdateSizeOnly(RHICmdList, TrackProxy->MeshData->Indices.Num());
					TrackProxy->VertexBuffer.UpdateSizeTyped<FNoPositionVertex>(RHICmdList, TrackProxy->MeshData->Vertices.Num());

					// Do the interpolation on a worker thread
					FGraphEventRef CompletionFence = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
					{

					}, GET_STATID(STAT_BufferUpdateTask), NULL, ENamedThreads::AnyThread);

					// Queue a command on the RHI thread that waits for the interpolation job and then uploads them to the GPU
					new (RHICmdList.AllocCommand<FRHICommandUpdateGeometryCacheBuffer>())FRHICommandUpdateGeometryCacheBuffer(
						CompletionFence,
						TrackProxy->VertexBuffer.VertexBufferRHI,
						TrackProxy->MeshData->Vertices.GetData(),
						TrackProxy->VertexBuffer.GetSizeInBytes(),
						TrackProxy->IndexBuffer.IndexBufferRHI,
						TrackProxy->MeshData->Indices.GetData(),
						TrackProxy->IndexBuffer.SizeInBytes());
				}
				else
				{
					TrackProxy->IndexBuffer.Update(TrackProxy->MeshData->Indices);
					TrackProxy->VertexBuffer.Update(TrackProxy->MeshData->Vertices);

					// Initialize both buffers the first frame
					if (TrackProxy->CurrentPositionBufferIndex == -1)
					{
						TrackProxy->PositonBuffers[0].Update(TrackProxy->MeshData->Vertices);
						TrackProxy->PositonBuffers[1].Update(TrackProxy->MeshData->Vertices);
						TrackProxy->CurrentPositionBufferIndex = 0;
					}
					else
					{
						TrackProxy->CurrentPositionBufferIndex++;
						TrackProxy->PositonBuffers[TrackProxy->CurrentPositionBufferIndex % 2].Update(TrackProxy->MeshData->Vertices);
					}
				}
			}
#endif

		}
	}
}

#if RHI_RAYTRACING
void FGeometryCacheSceneProxy::InitRayTracing(FRHICommandListBase& RHICmdList)
{
	for (FGeomCacheTrackProxy* Section : Tracks)
	{
		if (Section != nullptr)
		{
			const FVisibilitySample& VisibilitySample = Section->GetVisibilitySample(Time, bLooping);
			if (!VisibilitySample.bVisibilityState)
			{
				continue;
			}

			FRayTracingGeometryInitializer Initializer;
			Initializer.DebugName = RayTracingDebugName;
			const int PositionBufferIndex = Section->CurrentPositionBufferIndex != -1 ? Section->CurrentPositionBufferIndex % 2 : 0;
			Initializer.TotalPrimitiveCount = 0;
			Initializer.GeometryType = RTGT_Triangles;
			Initializer.bFastBuild = false;
			Initializer.bAllowCompaction = false;

			TArray<FRayTracingGeometrySegment> Segments;
			const FGeometryCacheMeshData* MeshData = Section->bNextFrameMeshDataSelected ? Section->NextFrameMeshData : Section->MeshData;
			for (const FGeometryCacheMeshBatchInfo& BatchInfo : MeshData->BatchesInfo)
			{
				FRayTracingGeometrySegment Segment;
				Segment.FirstPrimitive = BatchInfo.StartIndex / 3;
				Segment.NumPrimitives = BatchInfo.NumTriangles;
				Segment.VertexBuffer = Section->PositionBuffers[PositionBufferIndex].VertexBufferRHI;
				Segment.MaxVertices = Section->PositionBuffers[PositionBufferIndex].GetSizeInBytes() / Segment.VertexBufferStride; // conservative estimate
				Segments.Add(Segment);
				Initializer.TotalPrimitiveCount += BatchInfo.NumTriangles;
			}

			Initializer.Segments = Segments;

			// The geometry is not considered valid for initialization unless it has any triangles
			if (Initializer.TotalPrimitiveCount > 0)
			{
				Initializer.IndexBuffer = Section->IndexBuffer.IndexBufferRHI;
			}

			Section->RayTracingGeometry.SetInitializer(Initializer);
			Section->RayTracingGeometry.InitResource(RHICmdList);
			
			Section->bInitializedRayTracing = true;
		}
	}

}
#endif

void FGeometryCacheSceneProxy::UpdateSectionWorldMatrix(const int32 SectionIndex, const FMatrix& WorldMatrix)
{
	check(SectionIndex < Tracks.Num() && "Section Index out of range");
	Tracks[SectionIndex]->WorldMatrix = WorldMatrix;
}

void FGeometryCacheSceneProxy::ClearSections()
{
	Tracks.Empty();
	Scratch.Empty();
}

void FGeomCacheTrackProxy::InitRenderResources(FRHICommandListBase& RHICmdList, int32 NumVertices, int32 NumIndices)
{
	check(NumVertices);
	check(NumIndices);

	// Allocate verts
	TangentXBuffer.Init(NumVertices * sizeof(FPackedNormal));
	TangentZBuffer.Init(NumVertices * sizeof(FPackedNormal));
	TextureCoordinatesBuffer.Init(NumVertices * sizeof(FVector2f));
	ColorBuffer.Init(NumVertices * sizeof(FColor));

	PositionBuffers[0].Init(NumVertices * sizeof(FVector3f));
	PositionBuffers[1].Init(NumVertices * sizeof(FVector3f));
	CurrentPositionBufferIndex = -1;
	PositionBufferFrameIndices[0] = PositionBufferFrameIndices[1] = -1;
	PositionBufferFrameTimes[0] = PositionBufferFrameTimes[1] = -1.0f;

	// Allocate index buffer
	IndexBuffer.NumAllocatedIndices = NumIndices;
	IndexBuffer.NumValidIndices = 0;

	// Init vertex factory
	FGeometryCacheVertexVertexFactory::FDataType Data;
	FGeomCacheVertexFactory::InitComponentVF(&PositionBuffers[0], &PositionBuffers[1], &TangentXBuffer, &TangentZBuffer, &TextureCoordinatesBuffer, &ColorBuffer, Data);
	VertexFactory.SetData(RHICmdList, Data);

	// Enqueue initialization of render resource
	PositionBuffers[0].InitResource(RHICmdList);
	PositionBuffers[1].InitResource(RHICmdList);
	TangentXBuffer.InitResource(RHICmdList);
	TangentZBuffer.InitResource(RHICmdList);
	TextureCoordinatesBuffer.InitResource(RHICmdList);
	ColorBuffer.InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
	VertexFactory.InitResource(RHICmdList);

	bResourcesInitialized = true;
}

bool FGeomCacheTrackProxy::UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
	{
		return StreamableTrack->GetRenderResource()->UpdateMeshData(Time, bLooping, InOutMeshSampleIndex, OutMeshData);
	}
	return false;
}

bool FGeomCacheTrackProxy::GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
	{
		return StreamableTrack->GetRenderResource()->DecodeMeshData(SampleIndex, OutMeshData);
	}
	return false;
}

bool FGeomCacheTrackProxy::IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB)
{
	if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
	{
		return StreamableTrack->GetRenderResource()->IsTopologyCompatible(SampleIndexA, SampleIndexB);
	}
	return false;
}

const FVisibilitySample& FGeomCacheTrackProxy::GetVisibilitySample(float Time, const bool bLooping) const
{
	if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
	{
		return StreamableTrack->GetVisibilitySample(Time, bLooping);
	}
	return FVisibilitySample::InvisibleSample;
}

void FGeomCacheTrackProxy::FindSampleIndexesFromTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32& OutFrameIndex, int32& OutNextFrameIndex, float& InInterpolationFactor)
{
	if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
	{
		StreamableTrack->FindSampleIndexesFromTime(Time, bLooping, bIsPlayingBackwards, OutFrameIndex, OutNextFrameIndex, InInterpolationFactor);
	}
}

FGeomCacheVertexFactory::FGeomCacheVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
	: FGeometryCacheVertexVertexFactory(InFeatureLevel)
{

}

void FGeomCacheVertexFactory::InitDummyComponentVF(FGeometryCacheVertexVertexFactory::FDataType& OutData)
{
	OutData.bIsDummyData = true;
	
	// Provide empty buffers for all entries
	FGeomCacheVertexBuffer PositionBuffers[2];
	FGeomCacheTangentBuffer TangentXBuffer;
	FGeomCacheTangentBuffer TangentZBuffer;
	FGeomCacheVertexBuffer TextureCoordinatesBuffer;
	FGeomCacheColorBuffer ColorBuffer;
	InitComponentVF(&PositionBuffers[0], &PositionBuffers[1], &TangentXBuffer, &TangentZBuffer, &TextureCoordinatesBuffer, &ColorBuffer, OutData);

	// Clear out the pointer entries in OutData again
	OutData.PositionComponent.VertexBuffer = nullptr;
	for (FVertexStreamComponent& TextureStreamComponent : OutData.TextureCoordinates)
	{
		TextureStreamComponent.VertexBuffer = nullptr;
	}
	OutData.TangentBasisComponents[0].VertexBuffer = nullptr;
	OutData.TangentBasisComponents[1].VertexBuffer = nullptr;
	OutData.ColorComponent.VertexBuffer = nullptr;
	OutData.MotionBlurDataComponent.VertexBuffer = nullptr;
}

void FGeomCacheVertexFactory::InitComponentVF(const FVertexBuffer* PositionBuffer, const FVertexBuffer* MotionBlurDataBuffer, const FVertexBuffer* TangentXBuffer, const FVertexBuffer* TangentZBuffer, const FVertexBuffer* TextureCoordinateBuffer, const FVertexBuffer* ColorBuffer, FGeometryCacheVertexVertexFactory::FDataType& OutData)
{
	// Initialize the vertex factory's stream components.
	OutData.PositionComponent = FVertexStreamComponent(PositionBuffer, 0, sizeof(FVector3f), VET_Float3);

	OutData.TextureCoordinates.Empty();
	OutData.TextureCoordinates.Add(FVertexStreamComponent(TextureCoordinateBuffer, 0, sizeof(FVector2f), VET_Float2));
	OutData.TangentBasisComponents[0] = FVertexStreamComponent(TangentXBuffer, 0, sizeof(FPackedNormal), VET_PackedNormal);
	OutData.TangentBasisComponents[1] = FVertexStreamComponent(TangentZBuffer, 0, sizeof(FPackedNormal), VET_PackedNormal);
	OutData.ColorComponent = FVertexStreamComponent(ColorBuffer, 0, sizeof(FColor), VET_Color);
	OutData.MotionBlurDataComponent = FVertexStreamComponent(MotionBlurDataBuffer, 0, sizeof(FVector3f), VET_Float3);
}

void FGeomCacheVertexFactory::Init(FRHICommandListBase& RHICmdList, const FVertexBuffer* PositionBuffer, const FVertexBuffer* MotionBlurDataBuffer, const FVertexBuffer* TangentXBuffer, const FVertexBuffer* TangentZBuffer, const FVertexBuffer* TextureCoordinateBuffer, const FVertexBuffer* ColorBuffer)
{
	// Initialize the vertex factory's stream components.
	FDataType NewData;
	InitComponentVF(PositionBuffer, MotionBlurDataBuffer, TangentXBuffer, TangentZBuffer, TextureCoordinateBuffer, ColorBuffer, NewData);
	SetData(RHICmdList, NewData);
}

static FBufferRHIRef CreateGeomCacheIndexBuffer(FRHICommandListBase& RHICmdList, int32 NumAllocatedIndices)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateIndex<uint32>(TEXT("FGeomCacheIndexBuffer"), NumAllocatedIndices)
		.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	return RHICmdList.CreateBuffer(CreateDesc);
}

void FGeomCacheIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	IndexBufferRHI = CreateGeomCacheIndexBuffer(RHICmdList, NumAllocatedIndices);
	NumValidIndices = 0;

	if (IndexBufferRHI && NumAllocatedIndices)
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			IndexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}
}

void FGeomCacheIndexBuffer::ReleaseRHI()
{
	BufferSRV.SafeRelease();
	FIndexBuffer::ReleaseRHI();
}

void FGeomCacheIndexBuffer::Update(FRHICommandListBase& RHICmdList, const TArray<uint32>& Indices)
{
	SCOPE_CYCLE_COUNTER(STAT_IndexBufferUpdate);

	void* Buffer = nullptr;

	NumValidIndices = 0;

	// We only ever grow in size. Ok for now?
	bool bReallocate = false;
	if (Indices.Num() > NumAllocatedIndices)
	{
		NumAllocatedIndices = Indices.Num();
		IndexBufferRHI = CreateGeomCacheIndexBuffer(RHICmdList, NumAllocatedIndices);
		bReallocate = true;
	}

	if (Indices.Num() > 0)
	{
		// Copy the index data into the index buffer.
		Buffer = RHICmdList.LockBuffer(IndexBufferRHI, 0, NumAllocatedIndices * sizeof(uint32), RLM_WriteOnly);
	}


	if (Buffer)
	{
		FMemory::Memcpy(Buffer, Indices.GetData(), Indices.Num() * sizeof(uint32));
		NumValidIndices = Indices.Num();

		// Do not leave any of the index buffer memory uninitialized to prevent
		// the possibility of accessing vertex buffers out of bounds.
		uint32* LockedIndices = reinterpret_cast<uint32*>(Buffer);
		uint32 ValidIndexValue = Indices[0];
		for (int32 i = NumValidIndices; i < NumAllocatedIndices; ++i)
		{
			LockedIndices[i] = ValidIndexValue;
		}

		RHICmdList.UnlockBuffer(IndexBufferRHI);
	}

	if (bReallocate && IndexBufferRHI && NumAllocatedIndices)
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			IndexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}
}

void FGeomCacheIndexBuffer::UpdateSizeOnly(FRHICommandListBase& RHICmdList, int32 NewNumIndices)
{
	// We only ever grow in size. Ok for now?
	bool bReallocate = false;
	if (NewNumIndices > NumAllocatedIndices)
	{
		IndexBufferRHI = CreateGeomCacheIndexBuffer(RHICmdList, NewNumIndices);
		NumAllocatedIndices = NewNumIndices;
		NumValidIndices = 0;
		bReallocate = true;
	}

	if (bReallocate && IndexBufferRHI && NumAllocatedIndices)
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			IndexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}
}

static FBufferRHIRef CreateGeomCacheVertexBuffer(FRHICommandListBase& RHICmdList, const TCHAR* Name, int32 SizeInBytes)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FGeomCacheVertexBuffer"), SizeInBytes)
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	return RHICmdList.CreateBuffer(CreateDesc);
}

void FGeomCacheVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	VertexBufferRHI = CreateGeomCacheVertexBuffer(RHICmdList, TEXT("FGeomCacheVertexBuffer"), SizeInBytes);

	if (VertexBufferRHI && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
	}
}

void FGeomCacheVertexBuffer::ReleaseRHI()
{
	BufferSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

void FGeomCacheTangentBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	VertexBufferRHI = CreateGeomCacheVertexBuffer(RHICmdList, TEXT("FGeomCacheTangentBuffer"), SizeInBytes);

	if (VertexBufferRHI && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8_SNORM));
	}
}

void FGeomCacheColorBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	VertexBufferRHI = CreateGeomCacheVertexBuffer(RHICmdList, TEXT("FGeomCacheColorBuffer"), SizeInBytes);

	if (VertexBufferRHI && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_B8G8R8A8));
	}
}

void FGeomCacheVertexBuffer::UpdateRaw(FRHICommandListBase& RHICmdList, const void* Data, int32 NumItems, int32 ItemSizeBytes, int32 ItemStrideBytes)
{
	SCOPE_CYCLE_COUNTER(STAT_VertexBufferUpdate);
	int32 NewSizeInBytes = ItemSizeBytes * NumItems;
	bool bCanMemcopy = ItemSizeBytes == ItemStrideBytes;

	// Preserve SRV format if the SRV needs to be recreated
	EPixelFormat Format = BufferSRV ? BufferSRV->GetDesc().Buffer.SRV.Format : PF_R32_FLOAT;

	bool bReallocate = false;
	if (NewSizeInBytes > SizeInBytes)
	{
		SizeInBytes = NewSizeInBytes;
		VertexBufferRHI = CreateGeomCacheVertexBuffer(RHICmdList, TEXT("FGeomCacheVertexBuffer"), SizeInBytes);
		bReallocate = true;
	}

	void* VertexBufferData = RHICmdList.LockBuffer(VertexBufferRHI, 0, SizeInBytes, RLM_WriteOnly);

	if (bCanMemcopy)
	{
		FMemory::Memcpy(VertexBufferData, Data, NewSizeInBytes);
	}
	else
	{
		int8* InBytes = (int8*)Data;
		int8* OutBytes = (int8*)VertexBufferData;
		for (int32 ItemId = 0; ItemId < NumItems; ItemId++)
		{
			FMemory::Memcpy(OutBytes, InBytes, ItemSizeBytes);
			InBytes += ItemStrideBytes;
			OutBytes += ItemSizeBytes;
		}
	}

	RHICmdList.UnlockBuffer(VertexBufferRHI);

	if (bReallocate && VertexBufferRHI && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
	}
}

void FGeomCacheVertexBuffer::UpdateSize(FRHICommandListBase& RHICmdList, int32 NewSizeInBytes)
{
	bool bReallocate = false;
	if (NewSizeInBytes > SizeInBytes)
	{
		SizeInBytes = NewSizeInBytes;
		VertexBufferRHI = CreateGeomCacheVertexBuffer(RHICmdList, TEXT("FGeomCacheVertexBuffer"), SizeInBytes);
		bReallocate = true;
	}

	if (bReallocate && VertexBufferRHI && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		BufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
	}
}
