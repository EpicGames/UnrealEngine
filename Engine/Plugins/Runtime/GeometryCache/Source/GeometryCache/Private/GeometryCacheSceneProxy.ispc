// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Vector.isph"

#define InterpolatePositionsVarying 1

export void InterpolatePositions(
	uniform FVector3f PositionA[],
	uniform FVector3f PositionB[],
	uniform FVector3f Scratch[],
	uniform int NumVerts,
	uniform float WeightA,
	uniform float WeightB)
{
#if InterpolatePositionsVarying
	foreach(i = 0 ... NumVerts)
	{
		const uniform int Index = extract(i, 0);
		FVector3f VecA = VectorLoad(PositionA + Index);
		FVector3f VecB = VectorLoad(PositionB + Index);
		FVector3f InterpResult = (VecA * WeightA) + (VecB * WeightB);
		VectorStore(Scratch + Index, InterpResult);
	}
#else
	for (uniform int i = 0; i < NumVerts; ++i)
	{
		uniform FVector3f VecA = PositionA[i];
		uniform FVector3f VecB = PositionB[i];
		Scratch[i] = (VecA * WeightA) + (VecB * WeightB);;
	}
#endif
}

#define InterpolateMotionVectorsVarying 1

export void InterpolateMotionVectors(
	uniform FVector3f PositionA[],
	uniform FVector3f PositionB[],
	uniform FVector3f Scratch[],
	uniform int NumVerts,
	uniform float WeightA,
	uniform float WeightB,
	uniform float MotionVectorScale)
{
#if InterpolateMotionVectorsVarying
	foreach(i = 0 ... NumVerts)
	{
		const uniform int Index = extract(i, 0);
		FVector3f VecA = VectorLoad(PositionA + Index);
		FVector3f VecB = VectorLoad(PositionB + Index);
		FVector3f InterpResult = ((VecA * WeightA) + (VecB * WeightB)) * MotionVectorScale;
		VectorStore(Scratch + Index, InterpResult);
	}
#else
	for (uniform int i = 0; i < NumVerts; ++i)
	{
		uniform FVector3f VecA = PositionA[i];
		uniform FVector3f VecB = PositionB[i];
		Scratch[i] = ((VecA * WeightA) + (VecB * WeightB)) * MotionVectorScale;
	}
#endif
}

#define InterpolateUVsVarying 1

export void InterpolateUVs(
	uniform FVector2f PositionA[],
	uniform FVector2f PositionB[],
	uniform FVector2f Scratch[],
	uniform int NumVerts,
	uniform float WeightA,
	uniform float WeightB)
{
#if InterpolateUVsVarying
	foreach(i = 0 ... NumVerts)
	{
		const uniform int Index = extract(i, 0);
		FVector2f VecA = VectorLoad(PositionA + Index);
		FVector2f VecB = VectorLoad(PositionB + Index);
		FVector2f InterpResult = (VecA * WeightA) + (VecB * WeightB);
		VectorStore(Scratch + Index, InterpResult);
	}
#else
	for (uniform int i = 0; i < NumVerts; ++i)
	{
		uniform FVector2f VecA = PositionA[i];
		uniform FVector2f VecB = PositionB[i];
		Scratch[i] = (VecA * WeightA) + (VecB * WeightB);;
	}
#endif
}

export void InterpolateColors(
	uniform uint8 InColorA[],
	uniform uint8 InColorB[],
	uniform uint8 Scratch[],
	uniform int NumVerts,
	uniform float WeightA,
	uniform float WeightB)
{
	foreach(i = 0 ... NumVerts)
	{
		uint8 ColorA = InColorA[i];
		uint8 ColorB = InColorB[i];

		float AScale = (float)ColorA;
		float BScale = (float)ColorB;

		float ScaledChannels = (AScale * WeightA) + ((BScale * WeightB) + 0.5f);

		uint8 ConvColor = (uint8)ScaledChannels;

		Scratch[i] = ConvColor;
	}
}

export void InterpolateTangents(
	uniform uint8 InTangentXA[],
	uniform uint8 InTangentXB[],
	uniform uint8 InTangentZA[],
	uniform uint8 InTangentZB[],
	uniform uint8 XScratch[],
	uniform uint8 ZScratch[],
	uniform int NumVerts,
	uniform float WeightA,
	uniform float WeightB)
{
	uint8 SignMask = 0x80u;

	foreach(i = 0 ... NumVerts)
	{
		uint8 TangentXABytes = InTangentXA[i];
		uint8 TangentXBBytes = InTangentXB[i];
		uint8 TangentZABytes = InTangentZA[i];
		uint8 TangentZBBytes = InTangentZB[i];

		TangentXABytes ^= SignMask;
		TangentXBBytes ^= SignMask;
		TangentZABytes ^= SignMask;
		TangentZBBytes ^= SignMask;

		float XAScale = (float)TangentXABytes;
		float XBScale = (float)TangentXBBytes;
		float ZAScale = (float)TangentZABytes;
		float ZBScale = (float)TangentZBBytes;

		float XScaledChannels = (XAScale * WeightA) + ((XBScale * WeightB) + 0.5f);
		float ZScaledChannels = (ZAScale * WeightA) + ((ZBScale * WeightB) + 0.5f);

		uint8 XConvChannels = (uint8)XScaledChannels;
		uint8 ZConvChannels = (uint8)ZScaledChannels;

		XConvChannels ^= SignMask;
		ZConvChannels ^= SignMask;

		XScratch[i] = XConvChannels;
		ZScratch[i] = ZConvChannels;
	}
}
