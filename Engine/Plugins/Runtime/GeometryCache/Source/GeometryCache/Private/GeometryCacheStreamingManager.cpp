// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheStreamingManager.h"

#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheModule.h"
#include "GeometryCacheTrackStreamable.h"
#include "RenderingThread.h"
#include "StreamingGeometryCacheData.h"
#include "Async/Async.h"
#include "Misc/ScopeRWLock.h"

DECLARE_CYCLE_STAT(TEXT("Update Resource Streaming"), STAT_UpdateResourceStreaming, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Wait Untill Requests Finished"), STAT_BlockTillAllRequestsFinished, STATGROUP_GeometryCache);

DECLARE_MEMORY_STAT(TEXT("IO Bandwidth"), STAT_IOBandwidth, STATGROUP_GeometryCache);

DEFINE_LOG_CATEGORY(LogGeoCaStreaming);

static TAutoConsoleVariable<float> CVarLookaheadSeconds(
	TEXT("GeometryCache.LookaheadSeconds"),
	5.0,
	TEXT("The amount of data (expressed in seconds of animation) to try and keep resident in advance for geometry caches. Note this works regardless of the playback direction."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarTrailingSeconds(
	TEXT("GeometryCache.TrailingSeconds"),
	2.5,
	TEXT("The amount of data (expressed in seconds of animation) to try and keep resident inverse to the playback direction for geometry caches."),
	ECVF_Scalability | ECVF_RenderThreadSafe);


struct FGeometryCacheStreamingManager : public IGeometryCacheStreamingManager
{
	/** Constructor, initializing all members */
	FGeometryCacheStreamingManager();

	virtual ~FGeometryCacheStreamingManager();

	// IStreamingManager interface
	virtual void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) override;
	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) override;
	virtual void CancelForcedResources() override;
	virtual void NotifyLevelChange() override;
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) override;
	virtual void AddLevel(class ULevel* Level) override;
	virtual void RemoveLevel(class ULevel* Level) override;
	virtual void NotifyLevelOffset(class ULevel* Level, const FVector& Offset) override;
	// End IStreamingManager interface

	// IGeometryCacheStreamingManager interface
	virtual void AddGeometryCache(UGeometryCacheTrackStreamable* Cache) override;
	virtual void RemoveGeometryCache(UGeometryCacheTrackStreamable* Cache) override;
	virtual bool IsManagedGeometryCache(const UGeometryCacheTrackStreamable* Cache) const override;
	virtual bool IsStreamingInProgress(const UGeometryCacheTrackStreamable* Cache) override;
	virtual void AddStreamingComponent(UGeometryCacheComponent* CacheComponent) override;
	virtual void RemoveStreamingComponent(UGeometryCacheComponent* CacheComponent) override;
	virtual bool IsManagedComponent(const UGeometryCacheComponent* CacheComponent) const override;
	virtual void PrefetchData(UGeometryCacheComponent* CacheComponent) override;
	virtual const uint8* MapChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex, uint32* OutChunkSize = nullptr) override;
	virtual void UnmapChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex) override;
	// End IGeometryCacheStreamingManager interface

private:

	void PrefetchDataInternal(UGeometryCacheComponent* CacheComponent);

	/** Geometry caches being managed. */
	TMap<UGeometryCacheTrackStreamable*, FStreamingGeometryCacheData*> StreamingGeometryCaches;

	/** Scene components currently running geometry streaming. */
	TArray<UGeometryCacheComponent*>	StreamingComponents;

	mutable FRWLock StreamingGeometryCachesLock;

	double LastTickTime;
};

FGeometryCacheStreamingManager::FGeometryCacheStreamingManager()
{
}

FGeometryCacheStreamingManager::~FGeometryCacheStreamingManager()
{
}

void FGeometryCacheStreamingManager::UpdateResourceStreaming(float DeltaTime, bool bProcessEverything)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateResourceStreaming)
	check(IsInGameThread());

	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	
	//Phase zero: Clear ChunksNeeded
	for (auto Iter = StreamingGeometryCaches.CreateIterator(); Iter; ++Iter)
	{
		FStreamingGeometryCacheData* &CacheData = Iter.Value();
		CacheData->ResetNeededChunks();
	}

	// First phase: gather all the chunks that need to be streamed from all playing instances
	for (UGeometryCacheComponent* Component : StreamingComponents)
	{
		UGeometryCache* GC = Component->GeometryCache;
		if (GC)
		{
			for (auto Track : GC->Tracks)
			{
				UGeometryCacheTrackStreamable* GCTS = Cast<UGeometryCacheTrackStreamable>(Track);
				if (GCTS)
				{
					FStreamingGeometryCacheData** DataPtr = StreamingGeometryCaches.Find(GCTS);
					if (DataPtr)
					{
						float RequestStartTime = Component->GetAnimationTime() - (Component->GetPlaybackDirection() * CVarTrailingSeconds.GetValueOnGameThread());

						// We currently simply stream the next 5 seconds in animation time. Note that due to the playback speed
						// in wall-time this may be longer/shorter. It would be easy enough to change... need to test what's better
						float RequestEndTime = RequestStartTime + Component->GetPlaybackDirection() * CVarLookaheadSeconds.GetValueOnGameThread();
						if (RequestStartTime > RequestEndTime)
						{
							float tmp = RequestEndTime;
							RequestEndTime = RequestStartTime;
							RequestStartTime = tmp;
						}

						TArray<int32> ChunksNeeded;
						GCTS->GetChunksForTimeRange(RequestStartTime, RequestEndTime, Component->IsLooping(), ChunksNeeded);

						for (int32 ChunkId : ChunksNeeded)
						{
							(*DataPtr)->AddNeededChunk(ChunkId);
						}
					}
				}
			}
		}
	}

	double ThisTickTime = FPlatformTime::Seconds();
	double Delta = ThisTickTime - LastTickTime;
	unsigned BandWidthSinceLastTick = IoBandwidth.Reset();
	//(unsigned)(BandWidthSinceLastTick / Delta);
	SET_MEMORY_STAT(STAT_IOBandwidth, BandWidthSinceLastTick);
	LastTickTime = ThisTickTime;

	// Second phase: Schedule any new request we discovered, evict old, ...
	for (auto Iter = StreamingGeometryCaches.CreateIterator(); Iter; ++Iter)
	{
		FStreamingGeometryCacheData* CacheData = Iter.Value();
		CacheData->UpdateStreamingStatus();
	}
}

int32 FGeometryCacheStreamingManager::BlockTillAllRequestsFinished(float TimeLimit, bool bLogResults)
{
	SCOPE_CYCLE_COUNTER(STAT_BlockTillAllRequestsFinished)

	int32 Result = 0;

	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	if (TimeLimit == 0.0f)
	{
		for (auto Iter = StreamingGeometryCaches.CreateIterator(); Iter; ++Iter)
		{
			Iter.Value()->BlockTillAllRequestsFinished();
		}
	}
	else
	{
		double EndTime = FPlatformTime::Seconds() + TimeLimit;
		for (auto Iter = StreamingGeometryCaches.CreateIterator(); Iter; ++Iter)
		{
			float ThisTimeLimit = static_cast<float>(EndTime - FPlatformTime::Seconds());
			if (ThisTimeLimit < .001f || // one ms is the granularity of the platform event system
				!Iter.Value()->BlockTillAllRequestsFinished(ThisTimeLimit))
			{
				Result = 1; // we don't report the actual number, just 1 for any number of outstanding requests
				break;
			}
		}
	}

	return Result;
}

void FGeometryCacheStreamingManager::CancelForcedResources()
{
}

void FGeometryCacheStreamingManager::NotifyLevelChange()
{
}

void FGeometryCacheStreamingManager::SetDisregardWorldResourcesForFrames(int32 NumFrames)
{
}

void FGeometryCacheStreamingManager::AddLevel(class ULevel* Level)
{
	check(IsInGameThread());
}

void FGeometryCacheStreamingManager::RemoveLevel(class ULevel* Level)
{
	check(IsInGameThread());
}

void FGeometryCacheStreamingManager::NotifyLevelOffset(class ULevel* Level, const FVector& Offset)
{
	check(IsInGameThread());
}

void FGeometryCacheStreamingManager::AddGeometryCache(UGeometryCacheTrackStreamable* Cache)
{
	check(IsInGameThread() || IsInAsyncLoadingThread());

	FRWScopeLock WriteLock(StreamingGeometryCachesLock, SLT_Write);
	FStreamingGeometryCacheData* &CacheData = StreamingGeometryCaches.FindOrAdd(Cache);
	if (CacheData == nullptr)
	{
		CacheData = new FStreamingGeometryCacheData(Cache);
	}
}

void FGeometryCacheStreamingManager::RemoveGeometryCache(UGeometryCacheTrackStreamable* Cache)
{
	check(IsInGameThread());

	// Reorganized to not hold the lock while calling FlushRenderingCommand()
	FStreamingGeometryCacheData* CacheData = nullptr;
	{
		FRWScopeLock WriteLock(StreamingGeometryCachesLock, SLT_Write);
		FStreamingGeometryCacheData** FoundCacheData = StreamingGeometryCaches.Find(Cache);
		if (FoundCacheData)
		{
			CacheData = *FoundCacheData;
			StreamingGeometryCaches.Remove(Cache);
		}
	}

	if (CacheData != nullptr)
	{
		check(CacheData != nullptr);
		delete CacheData;	// This will FlushRenderingCommand outside the lock.
	}
}

bool FGeometryCacheStreamingManager::IsManagedGeometryCache(const UGeometryCacheTrackStreamable* Cache) const
{
	check(IsInGameThread());

	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	return StreamingGeometryCaches.Contains(Cache);
}

bool FGeometryCacheStreamingManager::IsStreamingInProgress(const UGeometryCacheTrackStreamable* Cache)
{
	check(IsInGameThread());

	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	FStreamingGeometryCacheData** CacheData = StreamingGeometryCaches.Find(Cache);
	if (CacheData != nullptr)
	{
		check(*CacheData != nullptr);
		return (*CacheData)->IsStreamingInProgress();
	}
	return false;
}

void FGeometryCacheStreamingManager::AddStreamingComponent(UGeometryCacheComponent* CacheComponent)
{
	check(IsInGameThread());
	StreamingComponents.AddUnique(CacheComponent);
	// Prefetch some data for all the streaming tracks on the cache
	PrefetchData(CacheComponent);
}

void FGeometryCacheStreamingManager::RemoveStreamingComponent(UGeometryCacheComponent* CacheComponent)
{
	check(IsInGameThread());
	StreamingComponents.Remove(CacheComponent);
}

void FGeometryCacheStreamingManager::PrefetchDataInternal(UGeometryCacheComponent* CacheComponent)
{
	check(IsInGameThread());
	check(IsManagedComponent(CacheComponent));

	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	if (CacheComponent->GeometryCache)
	{
		for (UGeometryCacheTrack* Track : CacheComponent->GeometryCache->Tracks)
		{
			UGeometryCacheTrackStreamable* GCTS = Cast<UGeometryCacheTrackStreamable>(Track);
			if (GCTS != nullptr)
			{
				FStreamingGeometryCacheData** data = StreamingGeometryCaches.Find(GCTS);
				checkf(data != nullptr, TEXT("No data could be prefetched for a track becasue the track was not registered with the manager."));
				(*data)->PrefetchData(CacheComponent);
			}
		}
	}
}

void FGeometryCacheStreamingManager::PrefetchData(UGeometryCacheComponent* CacheComponent)
{
	if (IsInGameThread())
	{
		PrefetchDataInternal(CacheComponent);
	}
	else
	{
		// The prefetch doesn't need to be executed right now, so schedule it for the game thread
		FWeakObjectPtr WeakComponent(CacheComponent);
		AsyncTask(ENamedThreads::GameThread, [this, WeakComponent]()
		{
			UGeometryCacheComponent* Component = Cast<UGeometryCacheComponent>(WeakComponent.Get());
			if (Component)
			{
				PrefetchDataInternal(Component);
			}
		});
	}
}

bool FGeometryCacheStreamingManager::IsManagedComponent(const UGeometryCacheComponent* CacheComponent) const
{
	check(IsInGameThread());
	return StreamingComponents.Contains(CacheComponent);
}

const uint8* FGeometryCacheStreamingManager::MapChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex, uint32* OutChunkSize)
{
	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	FStreamingGeometryCacheData** data = StreamingGeometryCaches.Find(Track);
	if (data)
	{
		return (*data)->MapChunk(ChunkIndex, OutChunkSize);
	}

	UE_LOGF(LogGeoCaStreaming, Error, "Tried to map a chunk in an unregistered animation track.");
	return nullptr;
}

void FGeometryCacheStreamingManager::UnmapChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex)
{
	FRWScopeLock ReadLock(StreamingGeometryCachesLock, SLT_ReadOnly);
	FStreamingGeometryCacheData** data = StreamingGeometryCaches.Find(Track);
	if (data)
	{
		(*data)->UnmapChunk(ChunkIndex);
	}
}

IGeometryCacheStreamingManager &IGeometryCacheStreamingManager::Get()
{
	static FGeometryCacheStreamingManager Instance;
	return Instance;
}

void IGeometryCacheStreamingManager::Register()
{
	IStreamingManager::Get().AddStreamingManager(&IGeometryCacheStreamingManager::Get());
}

void IGeometryCacheStreamingManager::Unregister()
{
	IStreamingManager::Get().RemoveStreamingManager(&IGeometryCacheStreamingManager::Get());
}
