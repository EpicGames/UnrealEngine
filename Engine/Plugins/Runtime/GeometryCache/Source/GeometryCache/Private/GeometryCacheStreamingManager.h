// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ContentStreaming.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGeoCaStreaming, Verbose, All);

class UGeometryCacheTrackStreamable;
class UGeometryCacheComponent;

/**
* Contains a request to load chunks of a geometry cache
*/
struct FGeometryCacheRequest
{
	TArray<uint32>	RequiredIndices;
	bool			bPrioritiseRequest;
};

/**
* Note IStreamingManager is not really anything like an interface it contains code and members and whatnot.
* So we just play along here to keep the spirit of the existing audio and texture streaming managers.
*/
struct IGeometryCacheStreamingManager : public IStreamingManager
{
	IGeometryCacheStreamingManager() {}

	/** Virtual destructor */
	virtual ~IGeometryCacheStreamingManager() = default;

	/** Getter of the singleton */
	GEOMETRYCACHE_API static struct IGeometryCacheStreamingManager& Get();

	/** Streaming manager interface registration enrollment */
	GEOMETRYCACHE_API static void Register();

	/** Streaming manager interface registration removal */
	GEOMETRYCACHE_API static void Unregister();

	/** Adds a new cache to the streaming manager. */
	virtual void AddGeometryCache(UGeometryCacheTrackStreamable* Cache) = 0;

	/** Removes a cache from the streaming manager. */
	virtual void RemoveGeometryCache(UGeometryCacheTrackStreamable* Cache) = 0;

	/** Returns true if this is a cache is managed by the streaming manager. */
	virtual bool IsManagedGeometryCache(const UGeometryCacheTrackStreamable* Cache) const = 0;

	/** Returns true if this data for this track is currently streaming. */
	virtual bool IsStreamingInProgress(const UGeometryCacheTrackStreamable* Cache) = 0;

	/** Adds a new component to the streaming manager. */
	virtual void AddStreamingComponent(UGeometryCacheComponent* CacheComponent) = 0;

	/** Removes the component from the streaming manager. */
	virtual void RemoveStreamingComponent(UGeometryCacheComponent* CacheComponent) = 0;

	/** Prefetch data for the current state component. Data is automatically prefetched when initially registering the component
	 * this may be useful when the component has seeked etc.*/
	virtual void PrefetchData(UGeometryCacheComponent* CacheComponent) = 0;

	/** Returns true if this is a streaming component that is managed by the streaming manager. */
	virtual bool IsManagedComponent(const UGeometryCacheComponent* CacheComponent) const = 0;

	/**
	* Gets a pointer to a chunk of cached geometry data. Can be called from any thread.
	*
	* @param Track Animation track we want a chunk from
	* @param ChunkIndex	Index of the chunk we want
	* @param OutChunkSize If non-null tores the size in bytes of the chunk
	* @return Either the desired chunk or NULL if it's not loaded
	*/
	virtual const uint8* MapChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex, uint32* OutChunkSize = nullptr ) = 0;

	/**
	* Releases pointer to a chunk of cahed geometry data. Can be called from any thread.
	* Should be called for every call to MapChunk.
	*
	* @param Track Animation track we want a chunk from
	* @param ChunkIndex	Index of the chunk we want
	*/
	virtual void UnmapChunk(const UGeometryCacheTrackStreamable* Track, uint32 ChunkIndex) = 0;

	FThreadSafeCounter IoBandwidth;
};

