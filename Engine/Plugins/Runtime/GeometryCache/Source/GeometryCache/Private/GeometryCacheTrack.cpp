// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrack.h"
#include "GeometryCacheHelpers.h"
#include "UObject/AnimPhysObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheTrack)

const FGeometryCacheTrackSampleInfo FGeometryCacheTrackSampleInfo::EmptySampleInfo;
const FVisibilitySample FVisibilitySample::VisibleSample(true);
const FVisibilitySample FVisibilitySample::InvisibleSample(false);

UGeometryCacheTrack::UGeometryCacheTrack(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UObject(ObjectInitializer)
{
	NumMaterials = 0;
	Duration = 0;
}

const bool UGeometryCacheTrack::UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData)
{
	return false;
}

const bool UGeometryCacheTrack::UpdateMatrixData(const float Time, const bool bLooping, int32& InOutMatrixSampleIndex, FMatrix& OutWorldMatrix)
{
	// Retrieve sample index from Time
	const uint32 MatrixSampleIndex = FindSampleIndexFromTime(MatrixSampleTimes, Time, bLooping);

	// Update the Matrix and Index if MatrixSampleIndex is different from the stored InOutMatrixSampleIndex
	if (MatrixSampleIndex != InOutMatrixSampleIndex)
	{
		InOutMatrixSampleIndex = MatrixSampleIndex;
		OutWorldMatrix = MatrixSamples[MatrixSampleIndex];
		return true;
	}

	return false;
}

const bool UGeometryCacheTrack::UpdateBoundsData(const float Time, const bool bLooping, const bool bIsPlayingBackward, int32& InOutBoundsSampleIndex, FBox& OutBounds)
{
	// Fixme implement bounds in derived classes, should we make this abstract?
	check(false);
	return false;
}

UGeometryCacheTrack::~UGeometryCacheTrack()
{
	MatrixSamples.Empty();
	MatrixSampleTimes.Empty();
}

void UGeometryCacheTrack::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) >= FAnimPhysObjectVersion::GeometryCacheAssetDeprecation)
	{
		Super::Serialize(Ar);
	}

	Ar << MatrixSamples;
	Ar << MatrixSampleTimes;
	Ar << NumMaterials;
}

void UGeometryCacheTrack::SetMatrixSamples(const TArray<FMatrix>& Matrices, const TArray<float>& SampleTimes)
{
	// Copy Matrix samples and sample-times
	MatrixSamples.Append(Matrices);
	MatrixSampleTimes.Append(SampleTimes);
}

void UGeometryCacheTrack::AddMatrixSample(const FMatrix& Matrix, const float SampleTime)
{
	MatrixSamples.Add(Matrix);
	MatrixSampleTimes.Add(SampleTime);

	Duration = FMath::Max(Duration, SampleTime);
}

void UGeometryCacheTrack::SetDuration(float NewDuration)
{
	Duration = NewDuration;
}

float UGeometryCacheTrack::GetDuration()
{
	return Duration;
}

const float UGeometryCacheTrack::GetMaxSampleTime() const
{
	// If there are sample-times available return the (maximal) time from the last sample
	if ( MatrixSampleTimes.Num() > 0 )
	{
		return MatrixSampleTimes.Last();
	}

	// Otherwise no data/times available
	return 0.0f;	
}

const uint32 UGeometryCacheTrack::FindSampleIndexFromTime(const TArray<float>& SampleTimes, const float Time, const bool bLooping)
{
	// No index possible
	if (SampleTimes.Num() == 0 || SampleTimes.Num() == 1)
	{
		return 0;
	}

	// Modulo the incoming Time if the animation is played on a loop
	float SampleTime = Time;
	if (bLooping)
	{
		SampleTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
	}

	// Binary searching for closest (floored) SampleIndex 
	uint32 MinIndex = 0;
	uint32 MaxIndex = SampleTimes.Num() - 1;
	if (SampleTime >= SampleTimes[MaxIndex])
	{
		return MaxIndex;
	}
	else if (SampleTime <= SampleTimes[MinIndex])
	{
		return MinIndex;
	}

	while (MaxIndex > 0 && MaxIndex >= MinIndex)
	{
		uint32 Mid = (MinIndex + MaxIndex) / 2;
		if (SampleTime > SampleTimes[Mid])
		{
			MinIndex = Mid + 1;
		}
		else
		{
			MaxIndex = Mid - 1;
		}
	}

	return MinIndex;
}

void UGeometryCacheTrack::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Determine resource size from data that is serialized
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MatrixSamples.Num() * sizeof(FMatrix));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MatrixSampleTimes.Num() * sizeof(float));
}

const FGeometryCacheTrackSampleInfo& UGeometryCacheTrack::GetSampleInfo(float Time, bool bLooping)
{
	return FGeometryCacheTrackSampleInfo::EmptySampleInfo;
}

