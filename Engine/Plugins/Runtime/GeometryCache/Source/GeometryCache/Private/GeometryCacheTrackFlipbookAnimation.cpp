// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrackFlipbookAnimation.h"
#include "GeometryCacheMeshData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheTrackFlipbookAnimation)


GEOMETRYCACHE_API UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::UDEPRECATED_GeometryCacheTrack_FlipbookAnimation(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UGeometryCacheTrack(ObjectInitializer)
{
	NumMeshSamples = 0;
}

UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::~UDEPRECATED_GeometryCacheTrack_FlipbookAnimation()
{
	NumMeshSamples = 0;
	MeshSamples.Empty();
	MeshSampleTimes.Empty();
}

void UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Determine resource size according to what is actually serialized
	for (int32 SampleIndex = 0; SampleIndex < MeshSamples.Num(); ++SampleIndex )
	{
		MeshSamples[SampleIndex].GetResourceSizeEx(CumulativeResourceSize);
	}
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(MeshSamples));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MeshSampleTimes.Num() * sizeof(float));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(MeshSampleTimes));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(NumMeshSamples));
}

const bool UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData)
{
	// Retrieve sample index from Time
	const int32 MeshSampleIndex = FindSampleIndexFromTime(MeshSampleTimes, Time, bLooping);

	// Update the Vertices and Index if MeshSampleIndex is different from the stored InOutMeshSampleIndex
	if (MeshSampleIndex != InOutMeshSampleIndex)
	{
		OutMeshData = &MeshSamples[MeshSampleIndex];
		InOutMeshSampleIndex = MeshSampleIndex;

		return true;
	}

	return false;
}

const float UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::GetMaxSampleTime() const
{
	const float BaseTime = UGeometryCacheTrack::GetMaxSampleTime();

	if (MeshSampleTimes.Num() > 0)
	{
		const float MeshSampleTime = MeshSampleTimes.Last();
		return (BaseTime > MeshSampleTime) ? BaseTime : MeshSampleTime;
	}

	return BaseTime;
}

void UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime)
{
	MeshSamples.Add(MeshData);
	MeshSampleTimes.Add(SampleTime);
	NumMeshSamples++;

	// Store the total number of materials within this track
	if (MeshData.BatchesInfo.Num() > (int32)NumMaterials)
	{
		NumMaterials = MeshData.BatchesInfo.Num();
	}
}

void UDEPRECATED_GeometryCacheTrack_FlipbookAnimation::BeginDestroy()
{
	Super::BeginDestroy();
	NumMeshSamples = 0;
	MeshSamples.Empty();
	MeshSampleTimes.Empty();
}

