// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheStreamingManager.h"
#include "GeometryCacheModule.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCachePreprocessor.h"
#include "StreamingGeometryCacheData.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheTrackStreamable)

DECLARE_CYCLE_STAT(TEXT("Decode Mesh Frame"), STAT_UpdateMeshData, STATGROUP_GeometryCache);
DECLARE_CYCLE_STAT(TEXT("Encode Mesh Frame"), STAT_AddMeshSample, STATGROUP_GeometryCache);

#if WITH_EDITOR
/**
 * Creates a totally invalid UGeometryCacheTrackStreamable instance specially set up to be very large
 * and then tries to serialize it to smoke-test the serialization of large assets and bulk data.
 */
void UGeometryCacheTrackStreamable::TriggerSerializationCrash()
{
	const FString PackageName = TEXT("/Game/CrashTest/CrashTest");
	UPackage *AssetPackage = CreatePackage(*PackageName);
	EObjectFlags Flags = RF_Public | RF_Standalone;

	UGeometryCacheTrackStreamable* Track = NewObject<UGeometryCacheTrackStreamable>(AssetPackage, FName(*FString(TEXT("DeleteMe"))), Flags);

	// Change these to smoke test large asset serialization
	const int64 ChunkDataSize = 16 * 1024 * 1024ll; // 512 MB a chunk
	const int64 BulkDataSize = 6 * 1024 * 1024 * 1024ll; // 6 gigs of bulk data
	const int64 AssetSize = 1024 * 1024 * 1024ll; // 1 Gig for the individual asset

	// Add enough bulk data chunks to the requested size
	const int32 NumChunks = static_cast<int32>(FMath::DivideAndRoundUp(BulkDataSize, ChunkDataSize));
	for ( int Count=0; Count<NumChunks;Count++)
	{
		// Create a new chunk for this coded frame
		FStreamedGeometryCacheChunk* NewChunk = new(Track->Chunks)FStreamedGeometryCacheChunk();
		NewChunk->DataSize = ChunkDataSize;
		NewChunk->FirstFrame = 0;
		NewChunk->LastFrame = 0;

		NewChunk->BulkData.Lock(LOCK_READ_WRITE);
		uint8* NewChunkData = (uint8 *)NewChunk->BulkData.Realloc(ChunkDataSize);
		// We don't actually bother initializing all the memory just touch it in case 
		NewChunkData[0] = 0xFF;
		NewChunkData[ChunkDataSize-1] = 0xFF;
		NewChunk->BulkData.Unlock();
	}

	// Add enough sample info objects to blow the asset up to the requested size
	const int32 NumSamplesToAdd = static_cast<int32>(FMath::DivideAndRoundUp(AssetSize, (int64)sizeof(FGeometryCacheTrackStreamableSampleInfo)));
	Track->Samples.AddUninitialized(NumSamplesToAdd);

	FString PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	SaveArgs.Error = GLog;
	UPackage::SavePackage(AssetPackage, Track, *PackageFileName, SaveArgs);
}

FAutoConsoleCommand TriggerSerializationCrashCommand(
	TEXT("geomcache.TriggerBulkDataCrash"),
	TEXT("Test a crash searializing large bulk data object"),
	FConsoleCommandDelegate::CreateStatic(UGeometryCacheTrackStreamable::TriggerSerializationCrash)
);
#endif

/*-----------------------------------------------------------------------------
UCompressedGeometryCacheTrack
-----------------------------------------------------------------------------*/

GEOMETRYCACHE_API UGeometryCacheTrackStreamable::UGeometryCacheTrackStreamable(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UGeometryCacheTrack(ObjectInitializer)
{
	Codec = nullptr;
	Hash = 0;
}

UGeometryCacheTrackStreamable::~UGeometryCacheTrackStreamable()
{
	Chunks.Empty();
	Samples.Empty();
}

void UGeometryCacheTrackStreamable::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Determine resource size according to what is actually serialized
	for (int32 ChunksIndex = 0; ChunksIndex < Chunks.Num(); ++ChunksIndex)
	{
		Chunks[ChunksIndex].GetResourceSizeEx(CumulativeResourceSize);
	}
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(Chunks));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(FGeometryCacheTrackStreamableSampleInfo) * Samples.Num());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(Samples));
}

void UGeometryCacheTrackStreamable::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	int32 NumChunks = Chunks.Num();
	Ar << NumChunks;

	if (Ar.IsLoading())
	{
		Chunks.AddDefaulted(NumChunks);
	}

	for (int32 ChunkId = 0; ChunkId < NumChunks; ChunkId++)
	{
		Chunks[ChunkId].Serialize(Ar, this, ChunkId);
	}

	Ar << Samples;
	Ar << VisibilitySamples;
}

const bool UGeometryCacheTrackStreamable::UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData)
{
	checkf(false, TEXT("This function is removed. Calls should go through the FGeometryCacheTrackStreamableRenderResource class"));
	return false;
}

const bool UGeometryCacheTrackStreamable::UpdateBoundsData(const float Time, const bool bLooping, const bool bIsPlayingBackward, int32& InOutBoundsSampleIndex, FBox& OutBounds)
{
	int32 SampleIndex;
	int32 NextSampleIndex;
	float Interpolation;
	FindSampleIndexesFromTime(Time, bLooping, bIsPlayingBackward, SampleIndex, NextSampleIndex, Interpolation);
	FGeometryCacheTrackStreamableSampleInfo SampleInfo = GetSampleInfo(SampleIndex);
	FGeometryCacheTrackStreamableSampleInfo NextSampleInfo = GetSampleInfo(NextSampleIndex);

	// We take the combined bounding box of the two frames surrounding the current time
	// this ensures that even when we are interpolating between frames we have a bounding box
	// that always fully encloses the rendered mesh
	OutBounds = SampleInfo.BoundingBox + NextSampleInfo.BoundingBox;
	if (InOutBoundsSampleIndex != SampleIndex)
	{
		// We use the SampleIndex to uniquely identify the returned box. In theory this is not enough info (e.g. if looping or playing direction
		// influences the two frames we use to generate the box ) but it's probably ok in practice.
		InOutBoundsSampleIndex = SampleIndex;
		return true;
	}
	else
	{
		return false;
	}
}

const float UGeometryCacheTrackStreamable::GetMaxSampleTime() const
{
	float BaseTime = UGeometryCacheTrack::GetMaxSampleTime();
	BaseTime = FMath::Max(Samples.Last().SampleTime, BaseTime);
	return BaseTime;
}

#if WITH_EDITORONLY_DATA
void UGeometryCacheTrackStreamable::BeginCoding(UGeometryCacheCodecBase* InCodec, bool bForceSingleOptimization, bool bCalculateAndStoreMotionVectors, bool bOptimizeIndexBuffers)
{
	// Make sure any render resources are freed before coding
	ReleaseRenderResources();
	ReleaseResourcesFence.Wait();

	Codec = InCodec;
	if (bCalculateAndStoreMotionVectors)
	{
		Preprocessor = new FExplicitMotionVectorGeometryCachePreprocessor(
			new FOptimizeGeometryCachePreprocessor(
				new FCodecGeometryCachePreprocessor(this), bForceSingleOptimization, bOptimizeIndexBuffers
			)
		);
	}
	else
	{
		Preprocessor = new FOptimizeGeometryCachePreprocessor(new FCodecGeometryCachePreprocessor(this), bForceSingleOptimization, bOptimizeIndexBuffers);
	}
	Codec->BeginCoding(Chunks);

	VisibilitySamples.Empty();
}

bool UGeometryCacheTrackStreamable::EndCoding()
{
	// This needs to be deleted first so it flushes any buffered frames before
	// we call EndCoding on the codec
	delete Preprocessor;

	check(Codec != 0);
	Codec->EndCoding();
	InitializeRenderResources();

	if (Samples.Num() > 0)
	{
		StartSampleTime = FMath::Min(Samples[0].SampleTime, 0.0f);
	}

	if (ImportVisibilitySamples.Num())
	{
		float RangeStart = StartSampleTime;
		bool bVisible = false;
		for (int32 SampleIndex = 0; SampleIndex < ImportVisibilitySamples.Num(); ++SampleIndex)
		{
			const TPair<float, bool>& VisibilitySample = ImportVisibilitySamples[SampleIndex];

			if (SampleIndex == 0)
			{
				bVisible = VisibilitySample.Value;
				// Set start range to start of sequence if it's visible from the get-go
				RangeStart = bVisible ? StartSampleTime : VisibilitySample.Key;
			}			
			else if (bVisible != VisibilitySample.Value)
			{				
				TRange<float> VisibilityRange(RangeStart, ImportVisibilitySamples[SampleIndex].Key);
				FVisibilitySample Sample(bVisible);
				Sample.Range = VisibilityRange;
				VisibilitySamples.Add(Sample);
				
				bVisible = ImportVisibilitySamples[SampleIndex].Value;
				RangeStart = VisibilityRange.GetUpperBoundValue();
			}
			else if (SampleIndex == ImportVisibilitySamples.Num() - 1)
			{
				TRange<float> VisibilityRange(RangeStart, ImportVisibilitySamples[SampleIndex].Key);
				FVisibilitySample Sample(ImportVisibilitySamples[SampleIndex].Value);
				Sample.Range = VisibilityRange;
				VisibilitySamples.Add(Sample);

				bVisible = ImportVisibilitySamples[SampleIndex].Value;
				RangeStart = VisibilityRange.GetUpperBoundValue();	
			}
		}

		// Make sure the range of all visibility samples covers the entire track duration
		if (Samples.Num() > 0 && RangeStart < Samples.Last().SampleTime)
		{
			TRange<float> VisibilityRange(RangeStart, Samples.Last().SampleTime);
			FVisibilitySample Sample(bVisible);
			Sample.Range = VisibilityRange;
			VisibilitySamples.Add(Sample);	
		}
	}
	else
	{
		TRange<float> VisibilityRange(StartSampleTime, Samples.Num() > 0 ? Samples.Last().SampleTime : 0.0f);
		FVisibilitySample Sample(true);
		Sample.Range = VisibilityRange;

		VisibilitySamples.Add(Sample);
	}

	// Determine duration
	if (Samples.Num() > 1)
	{
		Duration = Samples.Last().SampleTime - Samples[0].SampleTime;
	}

	return Samples.Num() > 0;
}

void UGeometryCacheTrackStreamable::AddMeshSample(const FGeometryCacheMeshData& MeshData, const float SampleTime, bool bSameTopologyAsPrevious)
{
	check(Codec != 0);

	uint64 TmpHash = MeshData.GetHash();
	if (TmpHash)
	{
		Hash = CityHash64WithSeed((char*) &TmpHash, sizeof(uint64), Hash);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_AddMeshSample);
		//FGeometryCacheCodecEncodeArguments Args( MeshData, SampleTime, bSameTopologyAsPrevious);
		//Codec->CodeFrame(Args);
		Preprocessor->AddMeshSample(MeshData, SampleTime, bSameTopologyAsPrevious);
	}

	Duration = FMath::Max(Duration, SampleTime);

	// Store the total number of materials within this track
	if (MeshData.BatchesInfo.Num() > (int32)NumMaterials)
	{
		NumMaterials = MeshData.BatchesInfo.Num();
	}
}

void UGeometryCacheTrackStreamable::AddVisibilitySample(const bool bVisible, const float SampleTime)
{
	ImportVisibilitySamples.Add(TPair<float, bool>(SampleTime, bVisible));
}

#endif

void UGeometryCacheTrackStreamable::SetDuration(float NewDuration)
{
	// Make sure any render resources are freed before coding
	ReleaseRenderResources();
	ReleaseResourcesFence.Wait();
	Duration = NewDuration;
	InitializeRenderResources();
}

void UGeometryCacheTrackStreamable::ReleaseRenderResources()
{
	BeginReleaseResource(&RenderResource);
	ReleaseResourcesFence.BeginFence();
}

void UGeometryCacheTrackStreamable::BeginDestroy()
{
	Super::BeginDestroy();
	ReleaseRenderResources();
	Codec = nullptr;
	Chunks.Empty();
	Samples.Empty();
}

bool UGeometryCacheTrackStreamable::IsReadyForFinishDestroy()
{
	if (Super::IsReadyForFinishDestroy())
	{
		return ReleaseResourcesFence.IsFenceComplete();
	}
	else
	{
		return false;
	}
}

void UGeometryCacheTrackStreamable::FinishDestroy()
{
	IGeometryCacheStreamingManager::Get().RemoveGeometryCache(this);
	check(!RenderResource.IsInitialized());
	Super::FinishDestroy();
}

void UGeometryCacheTrackStreamable::PostLoad()
{
	Super::PostLoad();
	IGeometryCacheStreamingManager::Get().AddGeometryCache(this);
	InitializeRenderResources();
}

void UGeometryCacheTrackStreamable::PostInitProperties()
{
	Super::PostInitProperties();
	IGeometryCacheStreamingManager::Get().AddGeometryCache(this);
}

void UGeometryCacheTrackStreamable::GetChunksForTimeRange(float StartTime, float EndTime, bool bLooping, TArray<int32>& OutChunkIndexes)
{
	// An option would be to delegate this to the codec...
	// This may put more burden on codec implementors but could offer better performance as 
	// they may have more info (for example knowing that chunks never overlap etc...)

	// Nothing to do if there is no chuncks
	if (Chunks.Num() == 0)
	{
		return;
	}

	ensureMsgf(StartTime <= EndTime, TEXT("Start time %f is greater than end time %f. Animation duration is %f"), StartTime, EndTime, Duration);

	// Correct wrap around if not in looping mode
	if (!bLooping && EndTime < StartTime)
	{
		EndTime = StartTime;
	}

	// If the first sample is fairly offset in time and > prefetch time return set endtime to first sample time
	if (Samples.Num() && EndTime < Samples[0].SampleTime)
	{
		EndTime = Samples[0].SampleTime;
	}

	if (bLooping)
	{
		float IntervalDuration = EndTime - StartTime;
		// Correct interval duration if there is a wrap around
		if (IntervalDuration < 0.0f)
		{
			IntervalDuration = (Duration - StartTime) + EndTime;
		}

		// More than a whole loop just get the whole thing
		if (IntervalDuration >= Duration)
		{
			for (int32 ChunkID = 0; ChunkID < Chunks.Num(); ChunkID++)
			{
				OutChunkIndexes.Add(ChunkID);
			}
			return;
		}

		//Adjust times to loop
		StartTime = GeometyCacheHelpers::WrapAnimationTime(StartTime, Duration);
		EndTime = GeometyCacheHelpers::WrapAnimationTime(EndTime, Duration);

		//Wrapped around the loop boundaries?
		//Fetch as two separate non-looped pieces
		if (EndTime < StartTime)
		{
			GetChunksForTimeRange(StartTime, Duration, false, OutChunkIndexes);
			GetChunksForTimeRange(0.0f, EndTime, false, OutChunkIndexes);
			return;
		}
	}

	// TODO: optimize this with binary search or other...
	uint32 FirstFrame = FindSampleIndexFromTime(StartTime, false);
	uint32 LastFrame = FindSampleIndexFromTime(EndTime, false);

	for (int32 ChunkID = 0; ChunkID <Chunks.Num(); ChunkID++)
	{
		if (Chunks[ChunkID].FirstFrame <= static_cast<float>(LastFrame) && Chunks[ChunkID].LastFrame >= static_cast<float>(FirstFrame))
		{
			OutChunkIndexes.Add(ChunkID);
		}
	}
}

void UGeometryCacheTrackStreamable::FindSampleIndexesFromTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32 &OutFrameIndex, int32 &OutNextFrameIndex, float &InterpolationFactor)
{
	// No index possible
	if (Samples.Num() == 0 || Samples.Num() == 1)
	{
		OutFrameIndex = 0;
		OutNextFrameIndex = 0;
		InterpolationFactor = 0.0f;
		return;
	}

	OutFrameIndex = FindSampleIndexFromTime(Time, bLooping);

	if (bLooping)
	{
		OutNextFrameIndex = (OutFrameIndex + 1) % Samples.Num();
	}
	else
	{
		OutNextFrameIndex = FMath::Min(OutFrameIndex + 1, Samples.Num() - 1);
	}

	float FrameDuration = Samples[OutNextFrameIndex].SampleTime - Samples[OutFrameIndex].SampleTime;
	if ( FMath::IsNearlyZero(FrameDuration))
	{
		InterpolationFactor = 0.0f;
	}
	else
	{
		float CorrTime;

		if (bLooping)
		{
			CorrTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
		}
		else
		{
			CorrTime = FMath::Clamp(Time, 0.0f, Samples.Last().SampleTime);
		}

		float Delta = CorrTime - Samples[OutFrameIndex].SampleTime;
		InterpolationFactor = Delta / FrameDuration;
	}

	// If playing backwards the logical order of previous and next is reversed
	if (bIsPlayingBackwards)
	{
		int32 Tmp = OutFrameIndex;
		OutFrameIndex = OutNextFrameIndex;
		OutNextFrameIndex = Tmp;
		InterpolationFactor = 1.0f - InterpolationFactor;
	}
}

const uint32 UGeometryCacheTrackStreamable::FindSampleIndexFromTime(const float Time, const bool bLooping) const
{
	// No index possible
	if (Samples.Num() == 0 || Samples.Num() == 1)
	{
		return 0;
	}

	// Modulo the incoming Time if the animation is played on a loop
	float SampleTime = Time;
	if (bLooping)
	{
		SampleTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
	}

	SampleTime += StartSampleTime;

	// Binary searching for closest (floored) SampleIndex 
	int32 MinIndex = 0;
	int32 MaxIndex = Samples.Num() - 1;
	if (SampleTime >= Samples[MaxIndex].SampleTime)
	{
		return MaxIndex;
	}
	else if (SampleTime <= Samples[MinIndex].SampleTime)
	{
		return MinIndex;
	}

	while (MaxIndex > 0 && MaxIndex > MinIndex)
	{
		int32 Mid = (MinIndex + MaxIndex + 1) / 2;
		if (SampleTime < Samples[Mid].SampleTime)
		{
			MaxIndex = Mid - 1;
		}
		else
		{
			MinIndex = Mid;
		}
	}

	// This is a flooring binary search. Validate this as there were bugs in the old code
	check(Samples[MinIndex].SampleTime <= SampleTime);
	check(Samples[FMath::Min(MinIndex+1, Samples.Num()-1)].SampleTime >= SampleTime);

	return MinIndex;
}

const FGeometryCacheTrackStreamableSampleInfo& UGeometryCacheTrackStreamable::GetSampleInfo(int32 SampleID) const
{
	checkf(Samples.IsValidIndex(SampleID), TEXT("Invalid sample (frame) index."));
	return Samples[SampleID];
}

const FGeometryCacheTrackSampleInfo& UGeometryCacheTrackStreamable::GetSampleInfo(float Time, bool bLooping)
{
	return GetSampleInfo(FindSampleIndexFromTime(Time, bLooping));
}

const FVisibilitySample& UGeometryCacheTrackStreamable::GetVisibilitySample(float Time, const bool bLooping) const
{
	float SampleTime = Time;
	if (bLooping)
	{
		SampleTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
	}

	for (const FVisibilitySample& Sample : VisibilitySamples)
	{
		if (Sample.Range.Contains(SampleTime))
		{
			return Sample;
		}
	}
	ensure(VisibilitySamples.Num());
	return VisibilitySamples.Last();
}

bool UGeometryCacheTrackStreamable::GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData)
{
	if (!Codec)
	{
		return false;
	}

	const int32 SampleIndex = FindSampleIndexFromTime(Time, false);
	return GetMeshDataAtSampleIndex(SampleIndex, OutMeshData);
}

bool UGeometryCacheTrackStreamable::GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (!Codec || !Samples.IsValidIndex(SampleIndex))
	{
		return false;
	}

	// Fetch the mesh data directly bypassing the GeometryCacheStreamingManager
	FStreamingGeometryCacheData Data(this);
	Data.AddNeededChunk(SampleIndex);
	Data.UpdateStreamingStatus();
	Data.BlockTillAllRequestsFinished();

	uint32 BufferSize = 0;
	const uint8 *Buffer = Data.MapChunk(SampleIndex, &BufferSize);
	bool bResult = Codec->DecodeBuffer(Buffer, BufferSize, OutMeshData);

	Data.UnmapChunk(SampleIndex);

	return bResult;
}

uint64 UGeometryCacheTrackStreamable::GetHash() const
{
	return Hash;
}

/**
	This should be called on the game thread whenever anything has changed to the object state
	that needs to be synced with the rendering thread.
*/
void UGeometryCacheTrackStreamable::InitializeRenderResources()
{
	check(IsInGameThread());
	check(!RenderResource.IsInitialized());
	check(Codec != nullptr); // BeginCoding was not called ?!?
	RenderResource.InitGame(this);
	BeginInitResource(&RenderResource);
}

FGeometryCacheTrackStreamableRenderResource::FGeometryCacheTrackStreamableRenderResource()
{
	Codec = nullptr;
	Track = nullptr;
}

void FGeometryCacheTrackStreamableRenderResource::InitGame(UGeometryCacheTrackStreamable *SetTrack)
{
	check(IsInGameThread());
	check(!IsInitialized()); //Still alive on the renderer?!?
	Codec = SetTrack->Codec->CreateRenderState();
	Track = SetTrack;
}

void FGeometryCacheTrackStreamableRenderResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	Codec->InitRHI(RHICmdList);
}

void FGeometryCacheTrackStreamableRenderResource::ReleaseRHI()
{
	delete Codec;
	Track = nullptr;
}

bool FGeometryCacheTrackStreamableRenderResource::UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	int32 InMeshIndex = InOutMeshSampleIndex;
	int32 SampleIndexToDecode = Track->FindSampleIndexFromTime(Time, bLooping);
	if (SampleIndexToDecode == InOutMeshSampleIndex)
	{
		return false;
	}
	
	if (Codec)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateMeshData);
		FGeometryCacheCodecDecodeArguments Args(*this, Track->Chunks, SampleIndexToDecode, OutMeshData);
		Codec->DecodeSingleFrame(Args);
		InOutMeshSampleIndex = SampleIndexToDecode;
	}
	return InMeshIndex != InOutMeshSampleIndex;
}

bool FGeometryCacheTrackStreamableRenderResource::DecodeMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateMeshData);
	if (!Codec)
	{
		return false;
	}
	FGeometryCacheCodecDecodeArguments Args(*this, Track->Chunks, SampleIndex, OutMeshData);
	return Codec->DecodeSingleFrame(Args);
}

bool FGeometryCacheTrackStreamableRenderResource::IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB)
{
	return Codec->IsTopologyCompatible(SampleIndexA, SampleIndexB);
}

FArchive& operator<<(FArchive& Ar, FGeometryCacheTrackStreamableSampleInfo& Info)
{
	Ar << Info.SampleTime;
	Ar << Info.BoundingBox;
	Ar << Info.NumVertices;
	Ar << Info.NumIndices;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FVisibilitySample& Sample)
{
	Ar << Sample.Range;
	Ar << Sample.bVisibilityState;

	return Ar;
}


