// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrackTransformAnimation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheTrackTransformAnimation)


GEOMETRYCACHE_API UDEPRECATED_GeometryCacheTrack_TransformAnimation::UDEPRECATED_GeometryCacheTrack_TransformAnimation(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/) : UGeometryCacheTrack(ObjectInitializer)
{
}

void UDEPRECATED_GeometryCacheTrack_TransformAnimation::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	MeshData.GetResourceSizeEx(CumulativeResourceSize);
}

const bool UDEPRECATED_GeometryCacheTrack_TransformAnimation::UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData)
{
	// If InOutMeshSampleIndex equals -1 (first creation) update the OutVertices and InOutMeshSampleIndex
	if (InOutMeshSampleIndex == -1)
	{
		OutMeshData = &MeshData;
		InOutMeshSampleIndex = 0;
		return true;
	}

	return false;
}

void UDEPRECATED_GeometryCacheTrack_TransformAnimation::SetMesh(const FGeometryCacheMeshData& NewMeshData)
{
	MeshData = NewMeshData;
	NumMaterials = NewMeshData.BatchesInfo.Num();
}

