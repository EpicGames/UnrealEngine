// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreGlobals.h"

#define HUFFMAN_USE_UNALIGNED_READ	(PLATFORM_SUPPORTS_UNALIGNED_LOADS && PLATFORM_LITTLE_ENDIAN)
#define MINIMUM_BITS_AFTER_REFILL	56																// Minimum number of bits guaranteed to be available in the internal buffer after a buffer refill.

/**
A bit stream writer class for use with the Huffman coding.
This class allows coding arbitrary sized integers up to 32 bits in size. The bits are written in Little-endian order.
*/
class FHuffmanBitStreamWriter
{
public:

	/**
	Create a writer. The class will automatically grow it's internal byte buffer used to store the bits.
	*/
	FHuffmanBitStreamWriter()
	{
		Clear();
	}

	/**
	Reset the writer as if it was freshly created. All written data will be discarded.
	*/
	void Clear()
	{
		Bytes.Empty();
		bFlushed = false;
		BitBuffer = 0;
		BitBufferBits = 0;
		NumBits = 0;
	}

	/**
	Write an NumBits integer value to the stream.
	*/
	void Write(uint32 Bits, uint32 InNumBits)
	{
		checkf(bFlushed == false, TEXT("You cannot write to a stream that has already been closed"));
		check(Bits < (1u << InNumBits));

		BitBuffer |= (uint64)Bits << BitBufferBits;
		BitBufferBits += InNumBits;
		NumBits += InNumBits;

		while(BitBufferBits >= 8)
		{
			Bytes.Add((uint8)BitBuffer);
			BitBuffer >>= 8;
			BitBufferBits -= 8;
		}
	}

	/**
	Flush pending bits and align to byte.
	*/
	void Align()
	{
		// Round up to next byte by appending 0s.
		if (BitBufferBits)
		{
			Write(0, 8 - BitBufferBits);
		}
		check(BitBufferBits == 0);
	}

	/**
	Close the stream. This ensures the stream's underlying byte buffer is correctly
	flushed with all bits written to the stream.
	Once closed you cannot longer Write additional data on the stream.
	*/
	void Close()
	{
		Align();
		
		bFlushed = true;
	}

	/**
	Get the bytes corresponding to this stream. The destination will be overwritten with the
	bytes for this stream.
	This can only be called on a flushed stream.
	*/
	const TArray<uint8> &GetBytes()
	{
		checkf(bFlushed == true, TEXT("You can only get bytes on an stream that has been closed"));
		return Bytes;
	}

	/**
	Get the number of bytes written into the stream so far.
	*/
	uint32 GetNumBytes() const
	{
		return Bytes.Num();
	}

	/**
	Get the number of bits written into the stream so far.
	*/
	uint32 GetNumBits() const
	{
		return NumBits;
	}

private:
	TArray<uint8> Bytes;
	uint64 BitBuffer;
	uint32 BitBufferBits;
	uint32 NumBits;
	bool bFlushed;
};

/**
Helper class to keep track of the amount of bits written by the BitstreamWriter
*/
class FBitstreamWriterByteCounter
{
public:	
	FBitstreamWriterByteCounter(const FHuffmanBitStreamWriter* SetWriter) : StartNumBits(0), StartNumBytes(0), Writer(SetWriter)
	{
		StartNumBits = Writer->GetNumBits();
		StartNumBytes = Writer->GetNumBytes();
	}

	/**
	Returns the amount of bytes written since this object was constructed
	*/
	uint32 Read()
	{
		const bool bHasExtraBits = Writer->GetNumBits() != StartNumBits;
		return Writer->GetNumBytes() - StartNumBytes + (bHasExtraBits ? 1 : 0);
	}
private:
	uint32 StartNumBits;
	uint32 StartNumBytes;
	const FHuffmanBitStreamWriter* Writer;
};

class FHuffmanBitStreamReader
{
public:
	/**
	Initialize the stream for reading with the specified data bytes. The bitstream doesn't own the data, so the caller is responsible for keeping the data valid while reader is active.
	Additionally the caller is responsible for over-allocating the buffer by 16 bytes, so we can safely read the data as uint64s.
	16 bytes and not 8 bytes, because the input pointer can move 8 bytes past the data and perform a 8 byte read from there.
	*/
	FHuffmanBitStreamReader(const uint8* InBytes, uint32 InNumBytes)
	{
		Bytes = InBytes;
		NumBytes = InNumBytes;
		Reset();
	}

	/**
	Restart reading from the stream. Reading will begin anew from the start of the stream.
	*/
	void Reset()
	{
		BitBuffer = 0;
		BitBufferBits = 0;
		BytePos = 0;
	}

	/**
	Fill the internal bit buffer. Will not check against buffer bounds, so make sure the input buffer is large enough to read all the bits requested +16 bytes.
	After the call the bit buffer is guaranteed to contain at least MINIMUM_BITS_AFTER_REFILL valid bits.
	*/
	FORCEINLINE void Refill()
	{
		const uint8* BytesData = Bytes;
#if HUFFMAN_USE_UNALIGNED_READ
		// Branchless buffer refill
		checkSlow(BytePos + 7 < NumBytes);	// Make sure entire read uint64 is within buffer bounds
		BitBuffer |= *(const uint64*)(BytesData + BytePos) << BitBufferBits;
		BytePos += (63 - BitBufferBits) >> 3;
		BitBufferBits |= 56;
#else
		// Read one byte at a time. Doesn't require unaligned reads and is agnostic to endianness.
		while (BitBufferBits <= 56)
		{
			checkSlow(BytePos < NumBytes);
			uint8 Byte = BytesData[BytePos++];
			BitBuffer |= uint64(Byte) << BitBufferBits;
			BitBufferBits += 8;
		}
#endif
	}

	/**
	Read a the next bit from the stream.
	*/
	FORCEINLINE uint32 Read()
	{
		return Read(1);
	}

	/**
	Read the next NumBits from the stream.
	*/
	FORCEINLINE uint32 Read(uint32 NumBits)
	{
		Refill();
		uint32 Value = BitBuffer & ((1ull << NumBits) - 1ull);
		BitBuffer >>= NumBits;
		BitBufferBits -= NumBits;
		return Value;
	}

	/**
	Read the next NumBits from the stream without refilling the bit buffer.
	*/
	FORCEINLINE uint32 ReadNoRefill(uint32 NumBits)
	{
		uint32 Value = BitBuffer & ((1ull << NumBits) - 1ull);
		BitBuffer >>= NumBits;
		BitBufferBits -= NumBits;
		return Value;
	}

	/**
	Read the next NumBits from the stream without refilling the bit buffer.
	*/
	FORCEINLINE void SkipNoRefill(uint32 NumBits)
	{
		BitBuffer >>= NumBits;
		BitBufferBits -= NumBits;
	}

	/**
	Return the next NumBits in the stream but do not advance the read position.
	*/
	FORCEINLINE int32 Peek(uint32 NumBits)
	{
		Refill();
		return BitBuffer & ((1ull << NumBits) - 1ull);
	}

	/**
	Return the next NumBits in the stream but do not advance the read position without refilling the bit buffer.
	*/
	FORCEINLINE int32 PeekNoRefill(uint32 NumBits)
	{
		return BitBuffer & ((1ull << NumBits) - 1ull);
	}

	/**
	Get the total number of bytes in the stream.
	*/
	FORCEINLINE int32 GetNumBytes()
	{
		return static_cast<int32>(NumBytes);
	}
private:
	const uint8* Bytes;
	uint64 NumBytes;
	uint64 BitBuffer;
	uint32 BitBufferBits;
	uint32 BytePos;
};

