// Copyright Epic Games, Inc. All Rights Reserved.

#include "HuffmanTable.h"

/**
To read the algorithmic docs you need to understand the following terms

- Symbol: An input value we want to compress, 0,1,2,3,...
- Codeword: The Huffman code. A variable length bit pattern.
- Some symbols will get a codeword assigned others which never occurred will not get a codeword assigned
- Symbols which occur more often preferably get a shorter code word
*/


/**
	Reverses the bits of an unsigned integer of a given number of bits.
*/
static uint32 ReverseBits(uint32 Value, uint32 NumBits)
{
	Value = ((Value & 0x55555555u) << 1u) | ((Value & 0xAAAAAAAAu) >> 1u);
	Value = ((Value & 0x33333333u) << 2u) | ((Value & 0xCCCCCCCCu) >> 2u);
	Value = ((Value & 0x0F0F0F0Fu) << 4u) | ((Value & 0xF0F0F0F0u) >> 4u);
	Value = ((Value & 0x00FF00FFu) << 8u) | ((Value & 0xFF00FF00u) >> 8u);
	Value = (Value << 16u) | (Value >> 16u);
	return Value >> (32u - NumBits);
}

/**
	Calculate optimal length-limited Huffman code lengths using the package-merge algorithm.
	Reference: https://www.ics.uci.edu/~dan/pubs/LenLimHuff.pdf
*/
static void GenerateLengthLimitedHuffmanLengths(uint8* OutLengths, uint32* Frequencies, int32 NumSymbols, int32 MaxCodeLength)
{
	check(NumSymbols <= (1 << MaxCodeLength));
	check(NumSymbols <= HUFFMAN_MAX_CODES);
	check(MaxCodeLength <= HUFFMAN_MAX_CODE_LENGTH);

	FMemory::Memzero(OutLengths, NumSymbols);

	struct SortEntry
	{
		uint32 Freq;
		int32 Symbol;
	};

	// Gather symbols with frequency > 0
	TArray<SortEntry> SortedSymbols;
	SortedSymbols.Reserve(NumSymbols);

	int32 NonZeroIndex = 0;
	for (int32 Symbol = 0; Symbol < NumSymbols; Symbol++)
	{
		uint32 Freq = Frequencies[Symbol];
		if (Freq > 0)
		{
			NonZeroIndex = Symbol;
			SortedSymbols.Emplace(SortEntry{ Freq, Symbol });
		}
	}

	// Sort symbols by frequency
	SortedSymbols.Sort([](const SortEntry& A, const SortEntry& B) { return A.Freq < B.Freq; });

	// If there is only one symbol give it length 1 to avoid having to handle 0-length codes
	int32 NumUsedSymbols = SortedSymbols.Num();
	if (NumUsedSymbols == 1)
	{
		OutLengths[NonZeroIndex] = 1;
		return;
	}

	struct Node
	{
		int32	Symbol;
		uint32	Freq;
		Node*	Children[2];

		// Traverse tree and every time a leaf node is visited the associated symbol code length is incremented by 1.
		static void GenerateLengthLimitedHuffmanLengthsInner(Node* Node, uint8* OutCodeLengths)
		{
			if (Node->Symbol == INT32_MIN)
			{
				GenerateLengthLimitedHuffmanLengthsInner(Node->Children[0], OutCodeLengths);
				GenerateLengthLimitedHuffmanLengthsInner(Node->Children[1], OutCodeLengths);
			}
			else
			{
				OutCodeLengths[Node->Symbol]++;
			}
		}
	};

	// Loop over code lengths in ascending order. At every length merge pairs of nodes from the previous level into the current whenever it is cheaper.
	TArray<Node> Nodes;
	Nodes.InsertZeroed(0, NumUsedSymbols * MaxCodeLength * 2);

	int32 CurrentNodeIndex = 0;
	int32 PrevNodeStartIndex = 0;
	int32 NumPrevNodes = 0;
	for (int32 CodeLength = 1; CodeLength <= MaxCodeLength; CodeLength++)
	{
		int32 NumCurrentNodes = NumUsedSymbols;
		int32 PrevNodeIndex = PrevNodeStartIndex;
		PrevNodeStartIndex = CurrentNodeIndex;

		int32 NumWrittenNodes = 0;
		while (NumPrevNodes >= 2 || NumCurrentNodes > 0)
		{
			// Merge pair of nodes from previous level into current level when they have a combined Freq smaller than the next node in the current level
			if (NumCurrentNodes > 0 && (NumPrevNodes < 2 || SortedSymbols[NumUsedSymbols - NumCurrentNodes].Freq <= Nodes[PrevNodeIndex + 0].Freq + Nodes[PrevNodeIndex + 1].Freq))
			{
				// Select node from current level
				const SortEntry& Entry = SortedSymbols[NumUsedSymbols - NumCurrentNodes];
				Node& Node = Nodes[CurrentNodeIndex++];
				Node.Freq = Entry.Freq;
				Node.Symbol = Entry.Symbol;
				Node.Children[0] = nullptr;
				Node.Children[1] = nullptr;
				NumCurrentNodes--;
			}
			else
			{
				// Merge two nodes from the previous level
				Node& Node = Nodes[CurrentNodeIndex++];
				Node.Freq = Nodes[PrevNodeIndex + 0].Freq + Nodes[PrevNodeIndex + 1].Freq;
				Node.Symbol = INT32_MIN;
				Node.Children[0] = &Nodes[PrevNodeIndex + 0];
				Node.Children[1] = &Nodes[PrevNodeIndex + 1];
				NumPrevNodes -= 2;
				PrevNodeIndex += 2;
			}
			NumWrittenNodes++;
		}
		NumPrevNodes = NumWrittenNodes;
	}

	// Traverse final level to update symbol lengths
	check(CurrentNodeIndex <= NumUsedSymbols * MaxCodeLength * 2);
	int32 NumActive = 2 * NumUsedSymbols - 2;
	for (int32 i = 0; i < NumActive; i++)
	{
		Node::GenerateLengthLimitedHuffmanLengthsInner(&Nodes[PrevNodeStartIndex + i], OutLengths);
	}
}

/**
	Generate canonical Huffman codes from symbol lengths.
	Codes are reversed, so they are actually postfix free instead of prefix free.
	Postfix codes are needed when bits are parsed from the bottom.
*/
static void GenerateHuffmanCodes(uint16* OutCodes, const uint8* CodeLengths, int32 NumSymbols, int32 MaxCodeLength)
{
	int32 SymbolLengthHistogram[HUFFMAN_MAX_CODE_LENGTH + 1] = {};			// Histogram of code lengths
	int16 SymbolLists[(HUFFMAN_MAX_CODE_LENGTH + 1) * HUFFMAN_MAX_CODES] = {};	// List of symbols for every code length

	// Fill histogram and generate symbol lists for each code length
	for (int32 Symbol = 0; Symbol < NumSymbols; Symbol++)
	{
		int32 CodeLength = CodeLengths[Symbol];
		check(CodeLength <= HUFFMAN_MAX_CODE_LENGTH);
		SymbolLists[CodeLength * HUFFMAN_MAX_CODES + SymbolLengthHistogram[CodeLength]++] = static_cast<int16>(Symbol);
	}

	// Assign code words to symbols
	int32 NextCodeWord = 0;
	for (int32 CodeLength = 1; CodeLength <= MaxCodeLength; CodeLength++)
	{
		// Loop over symbols in ascending order of current code length and assign consecutive codes to them starting from 2x the last code of the previous code length.
		int32 NumSymbolsOfLength = SymbolLengthHistogram[CodeLength];
		for (int32 i = 0; i < NumSymbolsOfLength; i++)
		{
			int32 Symbol = SymbolLists[CodeLength * HUFFMAN_MAX_CODES + i];
			check(CodeLengths[Symbol] == CodeLength);
			OutCodes[Symbol] = static_cast<uint16>(ReverseBits(NextCodeWord++, CodeLength));	// Reverse bits to turn prefix codes into postfix codes
		}
		NextCodeWord <<= 1;
	}
}

/**
	Initialize a Huffman table for encoding values in the range [0..NumSymbols[
	The table will initially be in prepass mode.
*/
void FHuffmanEncodeTable::Initialize(int32 InNumSymbols)
{
	NumSymbols = InNumSymbols;
	SymbolFrequencies.Init(0, NumSymbols);
	CodewordSymbols.Init(0, NumSymbols);
	// There may be less codewords than symbols but never more so this is a safe bet for the size of this array.
	CodeWords.Init(FCodeWord(), NumSymbols);
	ResetFrequencies();
	bIsPrepass = true;
}

/**
	Serialize the Huffman table to the bitstream. Deserialize by calling FHuffmanDecodeTable.Initialise()
*/
void FHuffmanEncodeTable::Serialize(FHuffmanBitStreamWriter& Stream)
{
	Stream.Write(NumSymbols, HUFFMAN_SYMBOL_COUNT_BITS);

	// Write symbol lengths. We don't need to write the actual codes as they can be canonically reconstructed from the lengths using GenerateHuffmanCodes
	for (int32 Symbol = 0; Symbol < NumSymbols; Symbol++)
	{
		Stream.Write(CodeWords[Symbol].Length, HUFFMAN_MAX_CODE_LENGTH_BITS);
	}
}

/**
Toggle between prepass mode and encoding/decoding mode.
Note: Toggling will update internal data structures and may have take more time than setting a simple variable.
*/
void FHuffmanEncodeTable::SetPrepass(bool bInIsPrepass)
{
	// If we enabled prepass reset the histogram
	if (bIsPrepass == false && bInIsPrepass == true)
	{
		SymbolFrequencies.Init(0, NumSymbols);
		ResetFrequencies();
	}
	// If we disabled prepass calculate the tables
	if (bIsPrepass == true && bInIsPrepass == false)
	{
		uint8 SymbolLengths[HUFFMAN_MAX_CODES];
		uint16 SymbolCodes[HUFFMAN_MAX_CODES];
		GenerateLengthLimitedHuffmanLengths(SymbolLengths, SymbolFrequencies.GetData(), NumSymbols, HUFFMAN_MAX_CODE_LENGTH);
		GenerateHuffmanCodes(SymbolCodes, SymbolLengths, NumSymbols, HUFFMAN_MAX_CODE_LENGTH);

		CodeWords.Empty();
		CodeWords.SetNumUninitialized(NumSymbols);
		for (int32 i = 0; i < NumSymbols; i++)
		{
			CodeWords[i].Bits = SymbolCodes[i];
			CodeWords[i].Length = SymbolLengths[i];
		}

		// We don't need the frequency table anymore once we generated the tables
		SymbolFrequencies.Empty(0);
	}
	bIsPrepass = bInIsPrepass;
}

/**
	Reset the frequency of all symbols to zero.
*/
void FHuffmanEncodeTable::ResetFrequencies()
{
	for (int32 i = 0; i < SymbolFrequencies.Num(); i++)
	{
		SymbolFrequencies[i] = 0;
	}
}

void FHuffmanEncodeTable::Encode(FHuffmanBitStreamWriter& Stream, int32 Symbol)
{
	// Check if symbol is in the valid range
	check(Symbol >= 0);
	check(Symbol < NumSymbols);

	if (bIsPrepass)
	{
		SymbolFrequencies[Symbol]++;
	}
	else
	{
		FCodeWord& CodeWord = CodeWords[Symbol];
		check(CodeWord.Length > 0); // We try to encode a symbol with code length zero, this means a codeword we never saw during our prepass
								// if you only do a partial prepass you should probably use the MarkAllSymbols function to ensure
								// all symbols get assigned a code word
		// and was thus not assigned a codeword
		Stream.Write(CodeWord.Bits, CodeWord.Length);
	}
}

/**
	Initialize a Huffman table based on the serialized table.
	The table will initially be in encode/decode mode.
*/
void FHuffmanDecodeTable::Initialize(FHuffmanBitStreamReader &Stream)
{
	int32 NumSymbols = Stream.Read(HUFFMAN_SYMBOL_COUNT_BITS);
	check(NumSymbols <= HUFFMAN_MAX_CODES);

	uint8 SymbolLengths[HUFFMAN_MAX_CODES];

	// Read symbol code lengths
	for (int32 Symbol = 0; Symbol < NumSymbols; Symbol++)
	{
		SymbolLengths[Symbol] = static_cast<uint8>(Stream.Read(HUFFMAN_MAX_CODE_LENGTH_BITS));
	}

	// Generate canonical huffman codes from code lengths
	uint16 SymbolCodes[HUFFMAN_MAX_CODES];
	GenerateHuffmanCodes(SymbolCodes, SymbolLengths, NumSymbols, HUFFMAN_MAX_CODE_LENGTH);

	// Generate decode lookup table
	uint32 MaxCode = 1u << HUFFMAN_MAX_CODE_LENGTH;
	for(int32 Symbol = 0; Symbol < NumSymbols; Symbol++)
	{
		uint32 Length = SymbolLengths[Symbol];
		if(Length > 0)
		{
			uint32 Code = SymbolCodes[Symbol];
			uint32 Step = 1u << Length;
			
			FTableEntry Entry;
			Entry.Symbol = static_cast<int8>(Symbol);
			Entry.Length = static_cast<int8>(Length);

			// Put entry in at every position where the Length last bits are Code.
			do
			{
				TableEntries[Code] = Entry;
				Code += Step;
			} while (Code < MaxCode);
		}
	}
}
