// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HuffmanBitStream.h"

/**
	Encapsulated Huffman coding/decoding classes. To use these classes roughly follow the following sequence of calls

	// Create an encoding table
	FHuffmanEncodeTable Tab(MaxValue);

	// First pass, gather statistics about the data to code
	Tab.SetPrepass(true)
	for each value to code
		Tab.Encode(value)
	// Second pass output actual bits
	Tab.SetPrepass(false)
	for each value to code
		Tab.Encode(value)

	The generated bits are then saved to disk alongside the generated Huffman table.
	To decode the following is then done

	FHuffmanDecodeTable Tab(SerializedTable)
	for each value to decode
		value = Tab.Decode()

	To read the docs you need to understand two terms

	- Symbol: An input value we want to compress, 0,1,2,3,...
	- Codeword: The Huffman code. A variable length bit pattern.
	- Some symbols will get a codeword assigned others which never occurred will not get a codeword assigned
	- Symbols which occur more often preferably get a shorter code word
*/

#define HUFFMAN_MAX_CODES 256
#define HUFFMAN_MAX_CODE_LENGTH 11
#define HUFFMAN_MAX_CODE_LENGTH_BITS 4	// Number of bits needed to decode a value in the range [0..HUFFMAN_MAX_CODE_LENGTH]
#define HUFFMAN_SYMBOL_COUNT_BITS 9		// Number of bits needed to send the number of symbols

class FHuffmanEncodeTable
{
public:
	/**
		Initialize a Huffman table for encoding values in the range [0..NumSymbols[
		The table will initially be in prepass mode.
	*/
	void Initialize(int32 NumSymbols);

	/**
		Serialize the Huffman table to the bitstream, deserialize using FHuffmanDecodeTable::Initialize
	*/
	void Serialize(FHuffmanBitStreamWriter& Stream);

	/**
		Toggle between prepass mode and encoding mode.
		Note: Toggling will update internal data structures and may have take more time than setting a simple variable.
	*/
	void SetPrepass(bool bInIsPrepass);

	/**
		Check if the table is currently operating in prepass mode.
	*/
	bool IsPrepass()
	{
		return bIsPrepass;
	}

	/**
		Encode a symbol to the bitstream. If the table is in prepass mode no actual bits will be emitted to the stream.
	*/
	void Encode(FHuffmanBitStreamWriter& Stream, int32 Symbol);
private:
	// This has a sentinel element at index NumValues!! And thus contains NumValues+1 elements
	TArray<uint32> SymbolFrequencies;
	TArray<int32> CodewordSymbols;

	struct FCodeWord
	{
		int32 Bits;
		int32 Length;
	};

	TArray<FCodeWord> CodeWords; // For every symbol the corresponding huffcode bits
	int32 NumCodeWordsOfLength[HUFFMAN_MAX_CODE_LENGTH+1]; // Number of codewords with a given length, element 0 is unused
	int32 NumSymbols;

	void ResetFrequencies();
	bool bIsPrepass;
};

class FHuffmanDecodeTable
{
public:
	/**
		Initialize a Huffman decode table based on the serialized table.
	*/
	void Initialize(FHuffmanBitStreamReader& Stream);

	/**
		Decode the next symbol from the bitstream.
	*/
	FORCEINLINE int32 Decode(FHuffmanBitStreamReader &Stream)
	{
		Stream.Refill();
		const int32 Bits = Stream.PeekNoRefill(HUFFMAN_MAX_CODE_LENGTH);

		const int32 Length = TableEntries[Bits].Length;
		Stream.SkipNoRefill(Length);
		return TableEntries[Bits].Symbol;
	}

	/**
	Decode the next symbol from the bitstream without refilling the bit buffer.
	*/
	FORCEINLINE int32 DecodeNoRefill(FHuffmanBitStreamReader &Stream)
	{
		const int32 Bits = Stream.PeekNoRefill(HUFFMAN_MAX_CODE_LENGTH);
		const int32 Length = TableEntries[Bits].Length;
		Stream.SkipNoRefill(Length);
		return TableEntries[Bits].Symbol;
	}

private:
	struct FTableEntry
	{
		int8 Length;
		int8 Symbol;
	};

	FTableEntry TableEntries[1u << HUFFMAN_MAX_CODE_LENGTH];
};
