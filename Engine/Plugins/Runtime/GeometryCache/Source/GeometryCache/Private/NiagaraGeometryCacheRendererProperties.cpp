// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGeometryCacheRendererProperties.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraModule.h"
#include "NiagaraParameterStore.h"
#include "NiagaraRendererGeometryCache.h"

#include "Modules/ModuleManager.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Widgets/Text/STextBlock.h"
#include "GeometryCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraGeometryCacheRendererProperties)

#if WITH_EDITOR
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"
#include "AssetThumbnail.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraGeometryCacheRendererProperties"

TArray<TWeakObjectPtr<UNiagaraGeometryCacheRendererProperties>> UNiagaraGeometryCacheRendererProperties::GeometryCacheRendererPropertiesToDeferredInit;
FNiagaraVariable UNiagaraGeometryCacheRendererProperties::Particles_GeoCacheRotation;
FNiagaraVariable UNiagaraGeometryCacheRendererProperties::Particles_Age;
FNiagaraVariable UNiagaraGeometryCacheRendererProperties::Particles_GeoCacheIsEnabled;
FNiagaraVariable UNiagaraGeometryCacheRendererProperties::Particles_ArrayIndex;

FNiagaraGeometryCacheReference::FNiagaraGeometryCacheReference()
	: GeometryCache(nullptr)
	, GeometryCacheUserParamBinding(FNiagaraTypeDefinition(UObject::StaticClass()))
{
}

UMaterialInterface* FNiagaraGeometryCacheReference::ResolveMaterial(UGeometryCache* ResolvedCache, int32 MaterialIndex) const
{
	UMaterialInterface* Material = OverrideMaterials.IsValidIndex(MaterialIndex) ? OverrideMaterials[MaterialIndex] : nullptr;
	if (Material == nullptr && GeometryCache->Materials.IsValidIndex(MaterialIndex))
	{
		Material = GeometryCache->Materials[MaterialIndex];
	}

	for (const FNiagaraGeometryCacheMICOverride& MICOverride : MICOverrideMaterials)
	{
		if (MICOverride.OriginalMaterial == Material)
		{
			return MICOverride.ReplacementMaterial;
		}
	}

	return Material;
}

UNiagaraGeometryCacheRendererProperties::UNiagaraGeometryCacheRendererProperties()
{
	AttributeBindings =
	{
		&PositionBinding,
		&RotationBinding,
		&ScaleBinding,
		&ElapsedTimeBinding,
		&EnabledBinding,
		&RendererVisibilityTagBinding,
		&ArrayIndexBinding,
	};

	if (GeometryCaches.Num() == 0)
	{
		GeometryCaches.AddDefaulted();
	}
	GeometryCaches.Shrink();
}

void UNiagaraGeometryCacheRendererProperties::PostLoad()
{
	Super::PostLoad();
	PostLoadBindings(ENiagaraRendererSourceDataMode::Particles);
	MaterialParameters.ConditionalPostLoad();
	for (const FNiagaraGeometryCacheReference& GeoCache : GeometryCaches)
	{
		for (UMaterialInterface* Material : GeoCache.OverrideMaterials)
		{
			if (Material)
			{
				Material->ConditionalPostLoad();
			}
		}
		for (const FNiagaraGeometryCacheMICOverride& MICOverride : GeoCache.MICOverrideMaterials)
		{
			if (MICOverride.OriginalMaterial)
			{
				MICOverride.OriginalMaterial->ConditionalPostLoad();
			}

			if (MICOverride.ReplacementMaterial)
			{
				MICOverride.ReplacementMaterial->ConditionalPostLoad();
			}
		}
	}
}

template<typename T>
FNiagaraVariableAttributeBinding CreateDefaultBinding(FNiagaraVariable DefaultValue, const T& DefaultValueData)
{
	DefaultValue.SetValue(DefaultValueData);
	FNiagaraVariableAttributeBinding Binding;
	Binding.Setup(DefaultValue, DefaultValue);
	return Binding;
}

void UNiagaraGeometryCacheRendererProperties::InitBindings()
{
	if (!PositionBinding.IsValid())
	{
		PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
		ScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SCALE);
		RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
		ArrayIndexBinding = CreateDefaultBinding(Particles_ArrayIndex, 0);
		RotationBinding = CreateDefaultBinding(Particles_GeoCacheRotation, FVector3f(0, 0, 0));
		ElapsedTimeBinding = CreateDefaultBinding(Particles_Age, 0.0f);
		EnabledBinding = CreateDefaultBinding(Particles_GeoCacheIsEnabled, false);
	}
}

void UNiagaraGeometryCacheRendererProperties::InitDefaultAttributes()
{
	if (!Particles_GeoCacheRotation.IsValid())
	{
		Particles_GeoCacheRotation = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.GeoCacheRotation"));
	}
	if (!Particles_Age.IsValid())
	{
		Particles_Age = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.Age"));
	}
	if (!Particles_GeoCacheIsEnabled.IsValid())
	{
		Particles_GeoCacheIsEnabled = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Particles.GeoCacheIsEnabled"));
	}
	if (!Particles_ArrayIndex.IsValid())
	{
		Particles_ArrayIndex = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.GeoCacheArrayIndex"));
	}
}

void UNiagaraGeometryCacheRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			GeometryCacheRendererPropertiesToDeferredInit.Add(this);
		}
		else
		{
			InitDefaultAttributes();
			
			if (!PositionBinding.IsValid())
			{
				PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
			}
			if (!ScaleBinding.IsValid())
			{
				ScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SCALE);
			}
			if (!RendererVisibilityTagBinding.IsValid())
			{
				RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
			}
			if (!RotationBinding.IsValid())
			{
				RotationBinding = CreateDefaultBinding(Particles_GeoCacheRotation, FVector3f(0, 0, 0));
			}
			if (!ElapsedTimeBinding.IsValid())
			{
				ElapsedTimeBinding = CreateDefaultBinding(Particles_Age, 0.0f);
			}
			if (!EnabledBinding.IsValid())
			{
				EnabledBinding = CreateDefaultBinding(Particles_GeoCacheIsEnabled, false);
			}
			if (!ArrayIndexBinding.IsValid())
			{
				ArrayIndexBinding = CreateDefaultBinding(Particles_ArrayIndex, 0);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraGeometryCacheRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraGeometryCacheRendererProperties, SourceMode))
	{
		UpdateSourceModeDerivates(SourceMode, true);
	}

	// Update our MICs if we change material / material bindings
	//-OPT: Could narrow down further to only static materials
	if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraGeometryCacheRendererProperties, GeometryCaches)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraGeometryCacheRendererProperties, MaterialParameters)))
	{
		UpdateMICs();
	}
}
#endif// WITH_EDITORONLY_DATA

void UNiagaraGeometryCacheRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	InitDefaultAttributes();
	
	UNiagaraGeometryCacheRendererProperties* CDO = CastChecked<UNiagaraGeometryCacheRendererProperties>(StaticClass()->GetDefaultObject());
	CDO->InitRendererBaseBindings();
	CDO->PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
	CDO->ScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SCALE);
	CDO->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
	CDO->RotationBinding = CreateDefaultBinding(Particles_GeoCacheRotation, FVector3f(0, 0, 0));
	CDO->ElapsedTimeBinding = CreateDefaultBinding(Particles_Age, 0.0f);
	CDO->EnabledBinding = CreateDefaultBinding(Particles_GeoCacheIsEnabled, false);
	CDO->ArrayIndexBinding = CreateDefaultBinding(Particles_ArrayIndex, 0);

	for (TWeakObjectPtr<UNiagaraGeometryCacheRendererProperties>& WeakGeometryCacheRendererProperties : GeometryCacheRendererPropertiesToDeferredInit)
	{
		if (UNiagaraGeometryCacheRendererProperties* RedererProperties = WeakGeometryCacheRendererProperties.Get())
		{
			RedererProperties->InitRendererBaseBindings();
			if (!RedererProperties->PositionBinding.IsValid())
			{
				RedererProperties->PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
			}
			if (!RedererProperties->ScaleBinding.IsValid())
			{
				RedererProperties->ScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SCALE);
			}
			if (!RedererProperties->RendererVisibilityTagBinding.IsValid())
			{
				RedererProperties->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
			}
			if (!RedererProperties->RotationBinding.IsValid())
			{
				RedererProperties->RotationBinding = CreateDefaultBinding(Particles_GeoCacheRotation, FVector3f(0, 0, 0));
			}
			if (!RedererProperties->ElapsedTimeBinding.IsValid())
			{
				RedererProperties->ElapsedTimeBinding = CreateDefaultBinding(Particles_Age, 0.0f);
			}
			if (!RedererProperties->EnabledBinding.IsValid())
			{
				RedererProperties->EnabledBinding = CreateDefaultBinding(Particles_GeoCacheIsEnabled, false);
			}
			if (!RedererProperties->ArrayIndexBinding.IsValid())
			{
				RedererProperties->ArrayIndexBinding = CreateDefaultBinding(Particles_ArrayIndex, 0);
			}
		}
	}
}

FNiagaraRenderer* UNiagaraGeometryCacheRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer* NewRenderer = new FNiagaraRendererGeometryCache(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

void UNiagaraGeometryCacheRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* EmitterInstance, TArray<UMaterialInterface*>& MaterialInterfaces) const
{
	for (const FNiagaraGeometryCacheReference& Entry : GeometryCaches)
	{
		UGeometryCache* GeometryCache = ResolveGeometryCache(Entry, EmitterInstance);
		if (GeometryCache == nullptr)
		{
			continue;
		}

		const int32 MaxIndex = FMath::Max(GeometryCache->Materials.Num(), Entry.OverrideMaterials.Num());
		for (int i=0; i < MaxIndex; ++i)
		{
			if (UMaterialInterface* Material = Entry.ResolveMaterial(GeometryCache, i))
			{
				MaterialInterfaces.AddUnique(Material);
			}
		}
	}
}

void UNiagaraGeometryCacheRendererProperties::GetUsedMeshes(const FNiagaraEmitterInstance* InEmitter, TArray<UObject*>& OutMeshes) const
{
	for (const FNiagaraGeometryCacheReference& Cache : GeometryCaches)
	{
		if (UGeometryCache* GeometryCache = ResolveGeometryCache(Cache, InEmitter))
		{
			OutMeshes.AddUnique(GeometryCache);
		}
	}
}

void UNiagaraGeometryCacheRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);

	if (UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>())
	{
		for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
		{
			MaterialParamBinding.CacheValues(SrcEmitter);
		}
	}
}

bool UNiagaraGeometryCacheRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	for (const FNiagaraGeometryCacheReference& Entry : GeometryCaches)
	{
		FNiagaraVariable Variable = Entry.GeometryCacheUserParamBinding.Parameter;
		if (Variable.IsValid())
		{
			InParameterStore.AddParameter(Variable, false);
			bAnyAdded = true;
		}
	}

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		InParameterStore.AddParameter(MaterialParamBinding.GetParamMapBindableVariable(), false);
		bAnyAdded = true;
	}

	return bAnyAdded;
}

void UNiagaraGeometryCacheRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	Super::CacheFromCompiledData(CompiledData);

	UpdateSourceModeDerivates(SourceMode);
	UpdateMICs();

	InitParticleDataSetAccessor(PositionAccessor, CompiledData, PositionBinding);
	InitParticleDataSetAccessor(RotationAccessor, CompiledData, RotationBinding);
	InitParticleDataSetAccessor(ScaleAccessor, CompiledData, ScaleBinding);
	InitParticleDataSetAccessor(ElapsedTimeAccessor, CompiledData, ElapsedTimeBinding);
	InitParticleDataSetAccessor(EnabledAccessor, CompiledData, EnabledBinding);
	InitParticleDataSetAccessor(ArrayIndexAccessor, CompiledData, ArrayIndexBinding);
	InitParticleDataSetAccessor(VisTagAccessor, CompiledData, RendererVisibilityTagBinding);
	UniqueIDAccessor.Init(CompiledData, FName("UniqueID"));
}

#if WITH_EDITORONLY_DATA
const FSlateBrush* UNiagaraGeometryCacheRendererProperties::GetStackIcon() const
{
	return FSlateIconFinder::FindIconBrushForClass(GetClass());
}

void UNiagaraGeometryCacheRendererProperties::GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UObject*> Assets;
	for(const FNiagaraGeometryCacheReference& Entry : GeometryCaches)
	{
		UGeometryCache* GeometryCache = Entry.GeometryCache;
		if (GeometryCache)
		{
			Assets.Add(GeometryCache);
		}
	}

	CreateRendererWidgetsForAssets(Assets, InThumbnailPool, OutWidgets);
}

void UNiagaraGeometryCacheRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool) const
{
	TSharedRef<SWidget> DefaultGeoCacheTooltip = SNew(STextBlock)
		.Text(LOCTEXT("GeoCacheRenderer", "Geometry Cache Renderer"));
	
	TArray<TSharedPtr<SWidget>> RendererWidgets;
	if (GeometryCaches.Num() > 0)
	{
		GetRendererWidgets(InEmitter, RendererWidgets, AssetThumbnailPool);
	}
	
	for(int32 Index = 0; Index < GeometryCaches.Num(); Index++)
	{
		const FNiagaraGeometryCacheReference& Entry = GeometryCaches[Index];
		
		TSharedPtr<SWidget> TooltipWidget = DefaultGeoCacheTooltip;		
		// we make sure to reuse the asset widget as a thumbnail if the geometry cache is valid
		if(Entry.GeometryCache)
		{
			TooltipWidget = RendererWidgets[Index];
		}

		// we override the previous thumbnail tooltip with a text indicating user parameter binding, if it exists
		if(Entry.GeometryCacheUserParamBinding.Parameter.IsValid())
		{
			TooltipWidget = SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("GeoCacheBoundTooltip", "Geometry cache slot is bound to user parameter {0}"), FText::FromName(Entry.GeometryCacheUserParamBinding.Parameter.GetName())));
		}
		
		OutWidgets.Add(TooltipWidget);
	}

	if (GeometryCaches.Num() == 0)
	{
		OutWidgets.Add(DefaultGeoCacheTooltip);
	}
}

const TArray<FNiagaraVariable>& UNiagaraGeometryCacheRendererProperties::GetOptionalAttributes()
{
	static TArray<FNiagaraVariable> Attrs;
	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		Attrs.Add(SYS_PARAM_PARTICLES_SCALE);
		Attrs.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
		Attrs.Add(Particles_GeoCacheRotation);
		Attrs.Add(Particles_Age);
		Attrs.Add(Particles_GeoCacheIsEnabled);
		Attrs.Add(Particles_ArrayIndex);
	}
	return Attrs;
}

void UNiagaraGeometryCacheRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	OutInfo.Add(FText::FromString(TEXT("The geometry cache renderer is still an experimental feature.")));

	if (MaterialParameters.HasAnyBindings())
	{
		TArray<UMaterialInterface*> Materials;
		GetUsedMaterials(nullptr, Materials);
		MaterialParameters.GetFeedback(Materials, OutWarnings);
	}
}

TArray<FNiagaraVariable> UNiagaraGeometryCacheRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + MaterialParameters.AttributeBindings.Num());

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		BoundAttributes.AddUnique(MaterialParamBinding.GetParamMapBindableVariable());
	}

	for (const FNiagaraGeometryCacheReference& GeometryCache : GeometryCaches)
	{
		if (GeometryCache.GeometryCacheUserParamBinding.Parameter.IsValid())
		{
			BoundAttributes.AddUnique(GeometryCache.GeometryCacheUserParamBinding.Parameter);
		}
	}

	return BoundAttributes;
}

void UNiagaraGeometryCacheRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RenameVariable(OldVariable, NewVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RenameVariable(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
#endif
}

void UNiagaraGeometryCacheRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RemoveVariable(OldVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RemoveVariable(OldVariable, InEmitter, GetCurrentSourceMode());
#endif
}
#endif

void UNiagaraGeometryCacheRendererProperties::UpdateMICs()
{
#if WITH_EDITORONLY_DATA
	// Grab existing MICs so we can reuse and clear them out so they aren't applied during GetUsedMaterials
	TArray<TObjectPtr<UMaterialInstanceConstant>> MICMaterials;
	for (FNiagaraGeometryCacheReference& GeoCache : GeometryCaches)
	{
		for (const FNiagaraGeometryCacheMICOverride& MICOverride : GeoCache.MICOverrideMaterials)
		{
			MICMaterials.AddUnique(MICOverride.ReplacementMaterial);
		}
		GeoCache.MICOverrideMaterials.Reset();
	}

	// Gather materials and generate MICs
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(nullptr, Materials);

	UpdateMaterialParametersMIC(MaterialParameters, Materials, MICMaterials);

	// Push MIC overrides back into each geometry cache
	if (MICMaterials.Num() > 0)
	{
		for (FNiagaraGeometryCacheReference& GeoCache : GeometryCaches)
		{
			UGeometryCache* GeometryCache = ResolveGeometryCache(GeoCache, nullptr);
			if (GeometryCache == nullptr)
			{
				continue;
			}

			const int32 MaxIndex = FMath::Max(GeometryCache->Materials.Num(), GeoCache.OverrideMaterials.Num());
			for (int i=0; i < MaxIndex; ++i)
			{
				UMaterialInterface* OriginalMaterial = GeoCache.OverrideMaterials.IsValidIndex(i) ? GeoCache.OverrideMaterials[i] : nullptr;
				if (OriginalMaterial == nullptr && GeometryCache->Materials.IsValidIndex(i))
				{
					OriginalMaterial = GeometryCache->Materials[i];
				}
				if (OriginalMaterial == nullptr)
				{
					continue;
				}

				const int32 MICIndex = Materials.IndexOfByKey(OriginalMaterial);
				UMaterialInstanceConstant* MIC = MICMaterials.IsValidIndex(MICIndex) ? MICMaterials[MICIndex] : nullptr;
				if (MIC)
				{
					FNiagaraGeometryCacheMICOverride& MICOverride = GeoCache.MICOverrideMaterials.AddDefaulted_GetRef();
					MICOverride.OriginalMaterial = OriginalMaterial;
					MICOverride.ReplacementMaterial = MIC;
				}
			}
		}
	}
#endif
}

UGeometryCache* UNiagaraGeometryCacheRendererProperties::ResolveGeometryCache(const FNiagaraGeometryCacheReference& Entry, const FNiagaraEmitterInstance* Emitter)
{
	UGeometryCache* FoundCache = nullptr;

	FNiagaraVariable Variable = Entry.GeometryCacheUserParamBinding.Parameter;
	if (Variable.IsValid() && Emitter)
	{
		UGeometryCache* GeometryCache = Cast<UGeometryCache>(Emitter->GetRendererBoundVariables().GetUObject(Variable));
		if (GeometryCache)
		{
			FoundCache = GeometryCache;
		}
	}

	if (!FoundCache)
	{
		FoundCache = Entry.GeometryCache;
	}

	return FoundCache;
}

#undef LOCTEXT_NAMESPACE
