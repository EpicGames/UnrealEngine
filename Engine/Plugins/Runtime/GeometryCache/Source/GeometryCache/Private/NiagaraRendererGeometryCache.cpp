// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererGeometryCache.h"

#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "GeometryCache.h"
#include "GeometryCacheStreamingManager.h"
#include "GeometryCacheComponent.h"
#include "NiagaraGeometryCacheRendererProperties.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSystemInstance.h"
#include "Async/Async.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

static float GNiagaraGeometryComponentRenderPoolInactiveTimeLimit = 5;
static FAutoConsoleVariableRef CVarNiagaraComponentRenderPoolInactiveTimeLimit(
	TEXT("fx.Niagara.GeometryComponentRenderPoolInactiveTimeLimit"),
	GNiagaraGeometryComponentRenderPoolInactiveTimeLimit,
	TEXT("The time in seconds an inactive component can linger in the pool before being destroyed."),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////


FNiagaraRendererGeometryCache::FNiagaraRendererGeometryCache(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
{
	const UNiagaraGeometryCacheRendererProperties* Properties = CastChecked<const UNiagaraGeometryCacheRendererProperties>(InProps);
	ComponentPool.Reserve(Properties->ComponentCountLimit);
}

FNiagaraRendererGeometryCache::~FNiagaraRendererGeometryCache()
{
	// These should have been freed in DestroyRenderState_Concurrent
	check(ComponentPool.Num() == 0);
}

void FNiagaraRendererGeometryCache::Initialize(const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer::Initialize(InProps, Emitter, InController);

	const UNiagaraGeometryCacheRendererProperties* Properties = CastChecked<UNiagaraGeometryCacheRendererProperties>(InProps);

	// Iniitialize the cache -> material mappings
	MaterialRemapTable.SetNum(Properties->GeometryCaches.Num());

	for (int32 iCache=0; iCache < Properties->GeometryCaches.Num(); ++iCache)
	{
		const FNiagaraGeometryCacheReference& GeoCacheRef = Properties->GeometryCaches[iCache];
		UGeometryCache* GeoCache = UNiagaraGeometryCacheRendererProperties::ResolveGeometryCache(GeoCacheRef, Emitter);
		if (GeoCache == nullptr)
		{
			continue;
		}
		MaterialRemapTable[iCache].SetNum(GeoCache->Materials.Num());

		for ( int32 iMaterial=0; iMaterial < GeoCache->Materials.Num(); ++iMaterial)
		{
			UMaterialInterface* Material = GeoCacheRef.ResolveMaterial(GeoCache, iMaterial);

			MaterialRemapTable[iCache][iMaterial] =
				BaseMaterials_GT.IndexOfByPredicate(
					[Material](UMaterialInterface* UsedMaterial)
					{
						if (UsedMaterial == Material)
						{
							return true;
						}
						UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(UsedMaterial);
						return MID ? MID->Parent == Material : false;
					}
				);
		}
	}
}

void FNiagaraRendererGeometryCache::DestroyRenderState_Concurrent()
{
	// Rendering resources are being destroyed, but the component pool and their owner actor must be destroyed on the game thread
	AsyncTask(
		ENamedThreads::GameThread,
		[Pool_GT = MoveTemp(ComponentPool), Owner_GT = MoveTemp(SpawnedOwner)]()
		{
			for (const FComponentPoolEntry& PoolEntry : Pool_GT)
			{
				if (PoolEntry.Component.IsValid())
				{
					IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(PoolEntry.Component.Get());
					PoolEntry.Component->DestroyComponent();
				}
			}

			if (AActor* OwnerActor = Owner_GT.Get())
			{
				OwnerActor->Destroy();
			}
		}
	);
	SpawnedOwner.Reset();
}

int32 GetGeometryCacheIndex(TOptional<int32> DefaultCacheIndex, bool bCreateRandomIfUnassigned, const FNiagaraEmitterInstance* Emitter, const UNiagaraGeometryCacheRendererProperties* Properties, int32 ParticleIndex, int32 ParticleID)
{
	if (bCreateRandomIfUnassigned == false && !DefaultCacheIndex.IsSet())
	{
		return INDEX_NONE;
	}
	int32 CacheIndex = DefaultCacheIndex.Get(INDEX_NONE);
	if (CacheIndex == INDEX_NONE && Emitter->IsDeterministic())
	{
		int32 Seed = Properties->bAssignComponentsOnParticleID ? ParticleID : ParticleIndex;
		FRandomStream RandomStream = FRandomStream(Seed * 907633515U); // multiply the seed, otherwise we get very poor randomness for small seeds
		CacheIndex = RandomStream.RandRange(0, Properties->GeometryCaches.Num());
	}
	else if (CacheIndex == INDEX_NONE)
	{
		CacheIndex = FMath::RandRange(0, Properties->GeometryCaches.Num() - 1);
	}
	return CacheIndex % Properties->GeometryCaches.Num(); // wrap around if index is too large
}

UGeometryCacheComponent* FNiagaraRendererGeometryCache::CreateOrGetPooledComponent(
	const UNiagaraGeometryCacheRendererProperties* Properties,
	USceneComponent* AttachComponent,
	const FNiagaraEmitterInstance* Emitter,
	TOptional<int32> DefaultCacheIndex,
	uint32 ParticleIndex,
	int32 ParticleID,
	int32& InOutPoolIndex
)
{
	UGeometryCacheComponent* GeometryComponent = ComponentPool.IsValidIndex(InOutPoolIndex) ? ComponentPool[InOutPoolIndex].Component.Get() : nullptr;
	const bool bCreateNewComponent = !GeometryComponent || GeometryComponent->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed);

	const int32 CacheIndex = GetGeometryCacheIndex(DefaultCacheIndex, bCreateNewComponent, Emitter, Properties, ParticleIndex, ParticleID);
	if (Properties->GeometryCaches.IsValidIndex(CacheIndex))
	{
		const FNiagaraGeometryCacheReference& Entry = Properties->GeometryCaches[CacheIndex];
		UGeometryCache* Cache = UNiagaraGeometryCacheRendererProperties::ResolveGeometryCache(Entry, Emitter);
		if (!Cache)
		{
			return nullptr;
		}

		if (bCreateNewComponent)
		{
			// Determine the owner actor or spawn one
			AActor* OwnerActor = SpawnedOwner.Get();
			if (OwnerActor == nullptr)
			{
				OwnerActor = AttachComponent->GetOwner();
				if (OwnerActor == nullptr)
				{
					// NOTE: This can happen with spawned systems
					OwnerActor = AttachComponent->GetWorld()->SpawnActor<AActor>();
					OwnerActor->SetFlags(RF_Transient);
					SpawnedOwner = OwnerActor;
				}
			}

			// if we don't have a pooled component we create a new one from the template
			GeometryComponent = NewObject<UGeometryCacheComponent>(OwnerActor);
			IGeometryCacheStreamingManager::Get().AddStreamingComponent(GeometryComponent);
			GeometryComponent->SetFlags(RF_Transient);
			GeometryComponent->SetupAttachment(AttachComponent);
			GeometryComponent->RegisterComponent();
			GeometryComponent->AddTickPrerequisiteComponent(AttachComponent);
			GeometryComponent->SetLooping(Properties->bIsLooping);
			GeometryComponent->SetManualTick(true); // we want to tick the component with the delta time of the niagara sim
			GeometryComponent->SetCanEverAffectNavigation(Properties->bAffectsNavigation);
			GeometryComponent->SetCollisionEnabled(Properties->EnableCollision);

			if (Emitter->IsLocalSpace())
			{
				GeometryComponent->SetAbsolute(false, false, false);
			}
			else
			{
				GeometryComponent->SetAbsolute(true, true, true);
			}

			if (InOutPoolIndex >= 0)
			{
				// This should only happen if the component was destroyed externally
				ComponentPool[InOutPoolIndex].Component = GeometryComponent;
				ComponentPool[InOutPoolIndex].LastElapsedTime = 0;
			}
			else
			{
				// Add a new pool entry
				InOutPoolIndex = ComponentPool.Num();
				ComponentPool.AddDefaulted_GetRef().Component = GeometryComponent;
				ComponentPool[InOutPoolIndex].LastElapsedTime = 0;
			}
		}

		GeometryComponent->SetGeometryCache(Cache);

		// set the cache and override materials, since they might have changed from the last time we accessed the array index
		// if they did not change from last tick these calls should be no-ops.
		for (int32 i=0; i < MaterialRemapTable[CacheIndex].Num(); ++i)
		{
			const int32 iMaterial = MaterialRemapTable[CacheIndex][i];
			UMaterialInterface* Material = BaseMaterials_GT.IsValidIndex(iMaterial) ? BaseMaterials_GT[iMaterial] : nullptr;
			GeometryComponent->SetMaterial(i, Material);
		}
	}
	else if (bCreateNewComponent)
	{
		return nullptr;
	}

	return GeometryComponent;
}


/** Update render data buffer from attributes */
void FNiagaraRendererGeometryCache::PostSystemTick_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter)
{
	FNiagaraSystemInstance* SystemInstance = Emitter->GetParentSystemInstance();

	//Bail if we don't have the required attributes to render this emitter.
	const UNiagaraGeometryCacheRendererProperties* Properties = CastChecked<const UNiagaraGeometryCacheRendererProperties>(InProperties);
	if (!SystemInstance || !Properties || Properties->GeometryCaches.Num() == 0 || SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (Emitter->IsDisabledFromIsolation())
	{
		ResetComponentPool(true);
		return;
	}
#endif

	USceneComponent* AttachComponent = SystemInstance->GetAttachComponent();
	if (!AttachComponent)
	{
		// we can't attach the components anywhere, so just bail
		return;
	}

	if (Properties->MaterialParameters.HasAnyBindings())
	{
		ProcessMaterialParameterBindings(Properties->MaterialParameters, Emitter, MakeArrayView(BaseMaterials_GT));
	}

	const FNiagaraParameterStore& ParameterStore = Emitter->GetRendererBoundVariables();
	const FNiagaraBool DefaultEnabled = ParameterStore.GetParameterValueOrDefault(Properties->EnabledBinding.GetParamMapBindableVariable(), FNiagaraBool(true));
	const int32 DefaultVisTag = ParameterStore.GetParameterValueOrDefault(Properties->RendererVisibilityTagBinding.GetParamMapBindableVariable(), Properties->RendererVisibility);
	const TOptional<int32> DefaultArrayIndex = ParameterStore.GetParameterOptionalValue<int32>(Properties->ArrayIndexBinding.GetParamMapBindableVariable());
	const int32 DefaultUniqueID = INDEX_NONE;
	const FNiagaraPosition DefaultPosition = ParameterStore.GetParameterValueOrDefault(Properties->PositionBinding.GetParamMapBindableVariable(), FNiagaraPosition(ForceInit));
	const FVector3f DefaultRotation = ParameterStore.GetParameterValueOrDefault(Properties->RotationBinding.GetParamMapBindableVariable(), FVector3f::ZeroVector);
	const FVector3f DefaultScale = ParameterStore.GetParameterValueOrDefault(Properties->ScaleBinding.GetParamMapBindableVariable(), FVector3f::OneVector);
	const float DefaultElapsedTime = ParameterStore.GetParameterValueOrDefault(Properties->ElapsedTimeBinding.GetParamMapBindableVariable(), 0.0f);
	const float CurrentTime = static_cast<float>(AttachComponent->GetWorld()->GetRealTimeSeconds());

	const FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter(Emitter->IsLocalSpace());
	const bool bIsRendererEnabled = IsRendererEnabled(InProperties, Emitter);

	if (Properties->SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		const FNiagaraDataSet& Data = Emitter->GetParticleData();
		const FNiagaraDataBuffer& ParticleData = Data.GetCurrentDataChecked();
		FNiagaraDataSetReaderInt32<FNiagaraBool> EnabledReader = Properties->EnabledAccessor.GetReader(Data);
		FNiagaraDataSetReaderInt32<int32> VisTagReader = Properties->VisTagAccessor.GetReader(Data);
		FNiagaraDataSetReaderInt32<int32> ArrayIndexReader = Properties->ArrayIndexAccessor.GetReader(Data);
		FNiagaraDataSetReaderInt32<int32> UniqueIDReader = Properties->UniqueIDAccessor.GetReader(Data);

		const auto IsParticleEnabled =
			[&EnabledReader, &DefaultEnabled, &VisTagReader, &DefaultVisTag, Properties](int32 ParticleIndex)
			{
				return EnabledReader.GetSafe(ParticleIndex, DefaultEnabled) && (VisTagReader.GetSafe(ParticleIndex, DefaultVisTag) == Properties->RendererVisibility);
			};

		TMap<int32, int32> ParticlesWithComponents;
		TArray<int32> FreeList;
		if (Properties->bAssignComponentsOnParticleID && ComponentPool.Num() > 0)
		{
			FreeList.Reserve(ComponentPool.Num());

			// Determine the slots that were assigned to particles last frame
			TMap<int32, int32> UsedSlots;
			UsedSlots.Reserve(ComponentPool.Num());
			for (int32 EntryIndex = 0; EntryIndex < ComponentPool.Num(); ++EntryIndex)
			{
				FComponentPoolEntry& Entry = ComponentPool[EntryIndex];
				if (Entry.LastAssignedToParticleID >= 0)
				{
					UsedSlots.Emplace(Entry.LastAssignedToParticleID, EntryIndex);
				}
				else
				{
					FreeList.Add(EntryIndex);
				}
			}

			// Ensure the final list only contains particles that are alive and enabled
			ParticlesWithComponents.Reserve(UsedSlots.Num());
			for (uint32 ParticleIndex = 0; ParticleIndex < ParticleData.GetNumInstances(); ParticleIndex++)
			{
				int32 ParticleID = UniqueIDReader.GetSafe(ParticleIndex, DefaultUniqueID);
				int32 PoolIndex;
				if (UsedSlots.RemoveAndCopyValue(ParticleID, PoolIndex))
				{
					if (bIsRendererEnabled && IsParticleEnabled(ParticleIndex))
					{
						ParticlesWithComponents.Emplace(ParticleID, PoolIndex);
					}
					else
					{
						// Particle has disabled components since last tick, ensure the component for this entry gets deactivated before re-use
						USceneComponent* Component = ComponentPool[PoolIndex].Component.Get();
						if (Component && Component->IsActive())
						{
							Component->Deactivate();
							Component->SetVisibility(false, true);
						}
						FreeList.Add(PoolIndex);
						ComponentPool[PoolIndex].LastAssignedToParticleID = -1;
					}
				}
			}

			// Any remaining in the used slots are now free to be reclaimed, due to their particles either dying or having their component disabled
			for (TPair<int32, int32> UsedSlot : UsedSlots)
			{
				// Particle has died since last tick, ensure the component for this entry gets deactivated before re-use
				USceneComponent* Component = ComponentPool[UsedSlot.Value].Component.Get();
				if (Component && Component->IsActive())
				{
					Component->Deactivate();
					Component->SetVisibility(false, true);
				}
				FreeList.Add(UsedSlot.Value);
				ComponentPool[UsedSlot.Value].LastAssignedToParticleID = -1;
			}
		}

		const int32 MaxComponents = Properties->ComponentCountLimit;
		int32 ComponentCount = 0;

		for (uint32 ParticleIndex = 0; ParticleIndex < ParticleData.GetNumInstances(); ParticleIndex++)
		{
			if (!bIsRendererEnabled || !IsParticleEnabled(ParticleIndex))
			{
				// Skip particles that don't want a component
				continue;
			}

			int32 ParticleID = -1;
			int32 PoolIndex = -1;
			if (Properties->bAssignComponentsOnParticleID)
			{
				// Get the particle ID and see if we have any components already assigned to the particle
				ParticleID = UniqueIDReader.GetSafe(ParticleIndex, DefaultUniqueID);
				ParticlesWithComponents.RemoveAndCopyValue(ParticleID, PoolIndex);
			}

			if (PoolIndex == -1 && ComponentCount + ParticlesWithComponents.Num() >= MaxComponents)
			{
				// The pool is full and there aren't any unused slots to claim
				continue;
			}

			// Acquire a component for this particle
			if (PoolIndex == -1)
			{
				// Start by trying to pull from the pool
				if (!Properties->bAssignComponentsOnParticleID)
				{
					// We can just take the next slot
					PoolIndex = ComponentCount < ComponentPool.Num() ? ComponentCount : -1;
				}
				else if (FreeList.Num())
				{
					PoolIndex = FreeList.Pop(EAllowShrinking::No);
				}
			}

			TOptional<int32> DefaultCacheIndex = DefaultArrayIndex;
			if (ArrayIndexReader.IsValid())
			{
				DefaultCacheIndex = ArrayIndexReader.GetSafe(ParticleIndex, INDEX_NONE);
			}

			UGeometryCacheComponent* GeometryComponent = CreateOrGetPooledComponent(Properties, AttachComponent, Emitter, DefaultCacheIndex, ParticleIndex, ParticleID, PoolIndex);
			if (GeometryComponent == nullptr)
			{
				continue;
			}

			const auto PositionAccessor = Properties->PositionAccessor.GetReader(Data);
			const auto RotationAccessor = Properties->RotationAccessor.GetReader(Data);
			const auto ScaleAccessor = Properties->ScaleAccessor.GetReader(Data);
			const auto ElapsedTimeAccessor = Properties->ElapsedTimeAccessor.GetReader(Data);

			FVector Position = LwcConverter.ConvertSimulationPositionToWorld(PositionAccessor.GetSafe(ParticleIndex, DefaultPosition));
			FVector Scale = (FVector)ScaleAccessor.GetSafe(ParticleIndex, DefaultScale);
			FVector RotationVector = (FVector)RotationAccessor.GetSafe(ParticleIndex, DefaultRotation);
			FTransform Transform(FRotator(RotationVector.X, RotationVector.Y, RotationVector.Z), Position, Scale);
			GeometryComponent->SetRelativeTransform(Transform);

			// activate the component
			if (!GeometryComponent->IsActive())
			{
				GeometryComponent->ResetSceneVelocity();
				GeometryComponent->SetVisibility(true, true);
				GeometryComponent->Activate(false);
			}

			float ElapsedTime = ElapsedTimeAccessor.GetSafe(ParticleIndex, DefaultElapsedTime);

			float Duration = GeometryComponent->GetDuration();
			if (Properties->bIsLooping && ElapsedTime < 0)
			{
				ElapsedTime -= FMath::CeilToFloat(ElapsedTime / Duration) * Duration;
			}
			else if (Properties->bIsLooping && ElapsedTime > Duration)
			{
				ElapsedTime -= FMath::Floor(ElapsedTime / Duration) * Duration;
			}

			FComponentPoolEntry& PoolEntry = ComponentPool[PoolIndex];
			GeometryComponent->TickAtThisTime(ElapsedTime, true, ElapsedTime < PoolEntry.LastElapsedTime, Properties->bIsLooping);

			PoolEntry.LastElapsedTime = ElapsedTime;
			PoolEntry.LastAssignedToParticleID = ParticleID;
			PoolEntry.LastActiveTime = CurrentTime;

			++ComponentCount;
			if (ComponentCount >= MaxComponents)
			{
				// We've hit our prescribed limit
				break;
			}
		}

		// Go over the pooled components we didn't need this tick to see if we can destroy some and deactivate the rest
		if (ComponentCount < ComponentPool.Num())
		{
			for (int32 PoolIndex = 0; PoolIndex < ComponentPool.Num(); ++PoolIndex)
			{
				FComponentPoolEntry& PoolEntry = ComponentPool[PoolIndex];
				if (Properties->bAssignComponentsOnParticleID)
				{
					if (PoolEntry.LastAssignedToParticleID >= 0)
					{
						// This one's in use
						continue;
					}
				}
				else if (PoolIndex < ComponentCount)
				{
					continue;
				}

				USceneComponent* Component = PoolEntry.Component.Get();
				if (!Component || (CurrentTime - PoolEntry.LastActiveTime) >= GNiagaraGeometryComponentRenderPoolInactiveTimeLimit)
				{
					if (Component)
					{
						IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(PoolEntry.Component.Get());
						Component->DestroyComponent();
					}

					// destroy the component pool slot
					ComponentPool.RemoveAtSwap(PoolIndex, EAllowShrinking::No);
					--PoolIndex;
					continue;
				}
				else if (Component->IsActive())
				{
					Component->Deactivate();
					Component->SetVisibility(false, true);
				}
			}
		}
	}
	// Emitter mode we can skip this if we aren't enabled / visible
	else if (bIsRendererEnabled && DefaultVisTag == Properties->RendererVisibility)
	{
		int32 PoolIndex = ComponentPool.Num() > 0 ? 0 : -1;
		int32 UsedComponentCount = 0;
		if ( UGeometryCacheComponent* GeometryComponent = CreateOrGetPooledComponent(Properties, AttachComponent, Emitter, DefaultArrayIndex, 0, 0, PoolIndex))
		{
			const FVector Position = LwcConverter.ConvertSimulationPositionToWorld(DefaultPosition);
			const FVector Scale = FVector(DefaultScale);
			const FVector RotationVector = FVector(DefaultRotation);
			const FTransform Transform(FRotator(RotationVector.X, RotationVector.Y, RotationVector.Z), Position, Scale);
			GeometryComponent->SetRelativeTransform(Transform);

			// activate the component
			if (!GeometryComponent->IsActive())
			{
				GeometryComponent->ResetSceneVelocity();
				GeometryComponent->SetVisibility(true, true);
				GeometryComponent->Activate(false);
			}

			const float Duration = GeometryComponent->GetDuration();
			float ElapsedTime = DefaultElapsedTime;
			if (Properties->bIsLooping && ElapsedTime < 0)
			{
				ElapsedTime -= FMath::CeilToFloat(ElapsedTime / Duration) * Duration;
			}
			else if (Properties->bIsLooping && ElapsedTime > Duration)
			{
				ElapsedTime -= FMath::Floor(ElapsedTime / Duration) * Duration;
			}

			FComponentPoolEntry& PoolEntry = ComponentPool[PoolIndex];
			GeometryComponent->TickAtThisTime(ElapsedTime, true, ElapsedTime < PoolEntry.LastElapsedTime, Properties->bIsLooping);

			PoolEntry.LastElapsedTime = ElapsedTime;
			PoolEntry.LastAssignedToParticleID = INDEX_NONE;
			PoolEntry.LastActiveTime = CurrentTime;

			++UsedComponentCount;
		}

		// We just scrub all the components we don't use, if we were to turn on / off the components we may want to reconsider this
		while ( ComponentPool.Num() > UsedComponentCount )
		{
			FComponentPoolEntry& PoolEntry = ComponentPool.Last();
			if ( USceneComponent* Component = PoolEntry.Component.Get() )
			{
				IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(PoolEntry.Component.Get());
				Component->DestroyComponent();
			}
		}
	}
}

void FNiagaraRendererGeometryCache::OnSystemComplete_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter)
{
	ResetComponentPool(true);
}

void FNiagaraRendererGeometryCache::ResetComponentPool(bool bResetOwner)
{
	for (FComponentPoolEntry& PoolEntry : ComponentPool)
	{
		if (PoolEntry.Component.IsValid())
		{
			IGeometryCacheStreamingManager::Get().RemoveStreamingComponent(PoolEntry.Component.Get());
			PoolEntry.Component->DestroyComponent();
		}
	}
	ComponentPool.SetNum(0, EAllowShrinking::No);

	if (bResetOwner)
	{
		if (AActor* OwnerActor = SpawnedOwner.Get())
		{
			SpawnedOwner.Reset();
			OwnerActor->Destroy();
		}
	}
}
