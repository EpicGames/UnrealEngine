// Copyright Epic Games, Inc. All Rights Reserved.

#include "NormalCompression.h"

DEFINE_LOG_CATEGORY(LogGeoCaStreamingNormalCompression);

FNormalCoderSmith::FNormalCoderSmith()
{
	// Magic numbers, these give good results, resulting in a very small error in degrees with a relative small number of bins.
	NPhiValue = 120;
	float MaximumErrorDegrees = 1.1f; // 1.1 degrees max error

	float MaximumError = MaximumErrorDegrees * PI / 180.0f;
	GenerateNThetaTable(NPhiValue, MaximumError, NThetaTable);

	MoveFrameEncoded.SetIdentity();
	MoveFrameDecoded.SetIdentity();
}

void FNormalCoderSmith::GenerateNThetaTable(int32 NPhi, float MaxError, TArray<int32>& NTheta)
{
	// Initialize our table
	NThetaTable.SetNumUninitialized(NPhiValue);

	int32 TotalPoints = 0;
	for (int32 IndexJ = 0; IndexJ < NPhi; ++IndexJ) // Loop over every bin for Phi
	{
		NTheta[IndexJ] = CalcNTheta(IndexJ, MaxError); // Calculate the number of bins for Theta
		TotalPoints += NTheta[IndexJ];
	}

	// Make sure NPhi is not set too small for the required maximum error. If NTheta becomes 1 for each NPhi,
	// set NPhi higher or MaximumErrorDegrees lower.
	bool bEveryEntryIsOne = (TotalPoints == NPhi);	
	check(!bEveryEntryIsOne); 
}

int32 FNormalCoderSmith::GetTotalBinCount() const
{
	// Count the total number of bins we have, sum the NTheta for all Phis. This gives us the number of (J,K) combinations.	
	// The higher this number, the higher the precision, but the more bits potentially required to store these indices.
	int32 TotalPoints = 0;
	for (int32 IndexJ = 0; IndexJ < NPhiValue; ++IndexJ)
	{
		TotalPoints += NThetaTable[IndexJ];
	}
	return TotalPoints;
}

int32 FNormalCoderSmith::CalcNTheta(int32 JIndex, float MaxError)
{
	// According to page 3 of Smith et al.
	float Phi = static_cast<float>(JIndex) * PI / static_cast<float>(NPhiValue - 1);
	float Numerator = FMath::Cos(MaxError) - FMath::Cos(Phi) * FMath::Cos(Phi + PI / static_cast<float>(2 * (NPhiValue - 1)));
	float Denominator = FMath::Sin(Phi) * FMath::Sin(Phi + PI / static_cast<float>(2 * (NPhiValue - 1)));

	if (Denominator == 0.0f) // pole
	{
		return 1;
	}

	float Acos = FMath::Acos(Numerator / Denominator);
	if (Acos != Acos)  // pole
	{
		return 1;
	}

	int32 NTheta = FMath::CeilToInt(PI / Acos);
	return NTheta;
}

FNormalCoderSmith::FEncodedNormal FNormalCoderSmith::Encode(const FVector& Value)
{
	FVector Moved = MoveFrameEncode(Value); // Make it relative to the previously-seen vector, close to (0, 0, 1)
	FSphericalCoordinates Spherical = VectorToSpherical(Moved);

	int32 JIndex = 0;
	int32 KIndex = 0;

	// Calculate our bin indices
	JIndex = FMath::RoundToInt(Spherical.Phi * static_cast<float>(NPhiValue - 1) / PI);
	int32 NTheta = GetNtheta(JIndex);
	KIndex = PositiveModulo(FMath::RoundToInt(Spherical.Theta * static_cast<float>(NTheta) / (2 * PI)), NTheta);

	// Decode those indices again
	FSphericalCoordinates DecodedDelta;
	DecodedDelta.Phi = static_cast<float>(JIndex) * PI / static_cast<float>(NPhiValue - 1);
	DecodedDelta.Theta = static_cast<float>(KIndex) * 2 * PI / static_cast<float>(NTheta);

	// Convert back to absolute positions as our decoder will do
	FVector DecodedDeltaVector = SphericalToVector(DecodedDelta);
	FVector DecodedVector = UnmoveFrame(DecodedDeltaVector, MoveFrameEncoded); // This updates the transform matrix

	// Return our coded results
	return FEncodedNormal(JIndex, KIndex);
}

FVector FNormalCoderSmith::Decode(const FEncodedNormal& Value)
{
	// Decode our bin indices to spherical coordinates, according to page 2 of Smith et al.
	FSphericalCoordinates DecodedDelta;
	DecodedDelta.Phi = static_cast<float>(Value.JIndex) * PI / static_cast<float>(NPhiValue - 1);
	int32 NTheta = GetNtheta(Value.JIndex);
	DecodedDelta.Theta = static_cast<float>(Value.KIndex) * 2 * PI / static_cast<float>(NTheta);

	// Transform our coordinates relative to the previously-seen vector back to absolute values
	FVector DecodedDeltaVector = SphericalToVector(DecodedDelta);
	FVector DecodedVector = UnmoveFrame(DecodedDeltaVector, MoveFrameDecoded);

	return DecodedVector;
}

/** Convert from Cartesian vectors to spherical coordinates */
FNormalCoderSmith::FSphericalCoordinates FNormalCoderSmith::VectorToSpherical(const FVector& Vector)
{
	FVector NormalizedVector = Vector;
	NormalizedVector.Normalize();
	FVector2D UESpherical = NormalizedVector.UnitCartesianToSpherical();
	return FNormalCoderSmith::FSphericalCoordinates(static_cast<float>(UESpherical.X), static_cast<float>(UESpherical.Y));
}

/** Convert from spherical coordinates to Cartesian vectors */
FVector FNormalCoderSmith::SphericalToVector(const FNormalCoderSmith::FSphericalCoordinates& Spherical)
{
	FVector2D UESpherical(Spherical.Phi, Spherical.Theta);
	return UESpherical.SphericalToUnitCartesian();
}

FVector FNormalCoderSmith::MoveFrameEncode(const FVector& Value)
{
	return MoveFrameEncoded.GetTransposed().TransformVector(Value); // MovedVector = (MovedFrameEncoded)T * Value;
}

FVector FNormalCoderSmith::GetMatrixColumn(FMatrix& Matrix, int32 Column)
{
	// Helper, we actually need Matrix.GetRow() or should refactor our math a bit so we can use Matrix.GetColumn
	return FVector(Matrix.M[Column][0], Matrix.M[Column][1], Matrix.M[Column][2]); 
}

void FNormalCoderSmith::SetMatrixColumn(FMatrix& Matrix, int32 Column, const FVector& Value)
{
	// Helper, we actually need Matrix.SetRow() or should refactor our math a bit so we can use Matrix.GetColumn
	Matrix.M[Column][0] = Value.X; 
	Matrix.M[Column][1] = Value.Y;
	Matrix.M[Column][2] = Value.Z;
}

FVector FNormalCoderSmith::UnmoveFrame(const FVector& Moved, FMatrix& TransformMatrix)
{
	// Unmove our moved vector, i.e., convert from the relative to the absolute vector
	FVector Result = TransformMatrix.TransformVector(Moved); //TransformMatrix * Moved;

	FVector Column1;
	FVector Column2;
	FVector Column3;

	// Create a new transform matrix that rotates the absolute vector tot the top of the sphere (0, 0, 1)
	// This matrix will transform the next vector from its relative position close to the top back to its absolute. 
	UpdateRotationMatrix(Result, Column1, Column2, Column3);

	// Update the transform matrix
	SetMatrixColumn(TransformMatrix, 0, Column1);
	SetMatrixColumn(TransformMatrix, 1, Column2);
	SetMatrixColumn(TransformMatrix, 2, Column3);

	return Result;
}

void FNormalCoderSmith::UpdateRotationMatrix(const FVector& UnmovedVector, FVector& ColumnX, FVector& ColumnY, FVector& ColumnZ)
{
	// Update the transform matrix that rotates vector Result to the top of the sphere(0, 0, 1), so this transformation can be applied to
	// the next vector to get it as close as possible to the top of the sphere

	if (UnmovedVector == FVector(0.0f, 0.0f, 1.0f)) // Aligns with Z axis, take a shortcut
	{
		ColumnX = FVector(1.0f, 0.0f, 0.0f);
		ColumnY = FVector(0.0f, 1.0f, 0.0f);
		ColumnZ = FVector(0.0f, 0.0f, 1.0f);
		return;
	}

	// Calculate the rotation matrix. Method in the paper did not seem to work. This is a generic rotation between two vectors but 
	// optimized for the case where one of the vectors is always (0, 0, 1)
	float SizeTerm = static_cast<float>(1.0f / (UnmovedVector.Y * UnmovedVector.Y + UnmovedVector.X * UnmovedVector.X));
	float X = static_cast<float>(UnmovedVector.Y * UnmovedVector.Y * SizeTerm * (1.0f - UnmovedVector.Z) + UnmovedVector.Z);
	float Y = static_cast<float>((UnmovedVector.Y * -UnmovedVector.X) * SizeTerm * (1.0f - UnmovedVector.Z));
	float Z = static_cast<float>(-UnmovedVector.X);

	ColumnZ = UnmovedVector;
	ColumnX = FVector(X, Y, Z);
	ColumnY = FVector::CrossProduct(ColumnZ, ColumnX);
}

int32 FNormalCoderSmithTest::Denormalize(float Value)
{
	// From -1,1 to 0-255
	return static_cast<int32>((Value + 1) * 0.5f * 255.0f);
}

FIntVector FNormalCoderSmithTest::Denormalize(const FVector& Value)
{
	// From -1,1 to 0-255
	return FIntVector(Denormalize(static_cast<float>(Value.X)), Denormalize(static_cast<float>(Value.Y)), Denormalize(static_cast<float>(Value.Z)));
}

void FNormalCoderSmithTest::Test()
{
	// Loop over all possible spherical coordinates, encode and decode using codec and calculate differences.
	FNormalCoderSmith Coder;
	FIntVector MaximumDifference(0, 0, 0);
	FVector MaximumDifferenceSpherical(0, 0, 0);
	
	const float Step = 0.01f;

	for (float Theta = 0.0f; Theta < 2 * PI; Theta += Step)
	{
		for (float Phi = 0.0f; Phi < PI / 2; Phi += Step)
		{
			FVector Input = FNormalCoderSmith::SphericalToVector(FNormalCoderSmith::FSphericalCoordinates(Phi, Theta));
			
			// Encode and decode
			FNormalCoderSmith::FEncodedNormal Encoded = Coder.Encode(Input);
			FVector Decoded = Coder.Decode(Encoded);

			// Calculate difference between original and decoded
			FVector Difference = Decoded - Input;
			FIntVector ByteDifference = Denormalize(Decoded) - Denormalize(Input);

			// Register difference
			MaximumDifference.X = FMath::Max(MaximumDifference.X, FMath::Abs(ByteDifference.X));
			MaximumDifference.Y = FMath::Max(MaximumDifference.Y, FMath::Abs(ByteDifference.Y));
			MaximumDifference.Z = FMath::Max(MaximumDifference.Z, FMath::Abs(ByteDifference.Z));

			// Calculate difference between original and decoded in degrees
			FNormalCoderSmith::FSphericalCoordinates InputSpherical = FNormalCoderSmith::VectorToSpherical(Input);
			FNormalCoderSmith::FSphericalCoordinates DecodedSpherical = FNormalCoderSmith::VectorToSpherical(Decoded);
			FVector SphericalDifference = FVector(InputSpherical.Theta - DecodedSpherical.Theta, InputSpherical.Phi - DecodedSpherical.Phi, 0);

			// Register difference
			MaximumDifferenceSpherical.X = FMath::Max(MaximumDifferenceSpherical.X, FMath::Abs(SphericalDifference.X));
			MaximumDifferenceSpherical.Y = FMath::Max(MaximumDifferenceSpherical.Y, FMath::Abs(SphericalDifference.Y));

			UE_LOGF(LogGeoCaStreamingNormalCompression, Log, "Input: (%i, %i, %i), Encoded: (%i, %i), Decoded: (%i, %i, %i), Difference: (%i, %i, %i)\n",
				Denormalize(static_cast<float>(Input.X)),
				Denormalize(static_cast<float>(Input.Y)),
				Denormalize(static_cast<float>(Input.Z)),
				Encoded.JIndex,
				Encoded.KIndex,
				Denormalize(static_cast<float>(Decoded.X)),
				Denormalize(static_cast<float>(Decoded.Y)),
				Denormalize(static_cast<float>(Decoded.Z)),
				ByteDifference.X, ByteDifference.Y, ByteDifference.Z);
		}
	}

	UE_LOGF(LogGeoCaStreamingNormalCompression, Log, "Maximum difference: (%i, %i, %i)\n", MaximumDifference.X, MaximumDifference.Y, MaximumDifference.Z);
	UE_LOGF(LogGeoCaStreamingNormalCompression, Log, "Maximum difference spherical (theta, phi): (%.2f, %.2f)\n", MaximumDifferenceSpherical.X, MaximumDifferenceSpherical.Y);
}