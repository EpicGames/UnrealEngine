// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Math/Matrix.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGeoCaStreamingNormalCompression, Verbose, All);

/** Normal codec according to Smith et al.
    Codes a unit vector as two indices indexing a quantized unit sphere and exploits coherence between successive coded vectors. 
    See the paper for details on the algorithm [Smith J., Petrova G., and Schaefer S. 2012. Encoding normal vectors using optimized 
	spherical coordinates. Computers & Graphics 36, 5, 360-365.].
	States they have better complexity/rate/quality properties than Octa (e.g., Griffith et al.) and Sextant (Deering et al.).
	
	Initial experiment and port to UnrealEngine, lots that can be optimized.
*/
class FNormalCoderSmith
{
public:
	/** Number of bins spanning the polar range. Determines the precision of the coded results */
	int32 NPhiValue;
	/** Table with number of bins for each NPhi/polar index value. Small values close to the poles, largest at the equator. */
	TArray<int32> NThetaTable;
	/** Encoder transform matrix, transforms a vector from its absolute position to its position relative to the previously-seen 
		vector. This gets the relative vector as close as possible to the top. */
	FMatrix MoveFrameEncoded;
	/** Decoder transform matrix, transforms a vector from its absolute position to its position relative to the previously-seen 
	    vector. This gets the relative vector as close as possible to the top. */
	FMatrix MoveFrameDecoded;   

	/** Coded normal representation, two indices indexing a bin on the sphere */
	struct FEncodedNormal
	{
		/** Index J, indexing a bin over the range of elevation/Phi*/
		int32 JIndex;
		/** Index K, indexing a bin over the range of azimuth/Theta*/
		int32 KIndex;
		
		FEncodedNormal(int32 SetJIndex, int32 SetKIndex) : JIndex(SetJIndex), KIndex(SetKIndex)
		{
		}
	};

	/** Spherical coordinates */
	struct FSphericalCoordinates
	{
		float Phi;
		float Theta;

		FSphericalCoordinates() : Phi(0.0f), Theta(0.0f)
		{
		}

		FSphericalCoordinates(float SetPhi, float SetTheta) : Phi(SetPhi), Theta(SetTheta)
		{
		}		
	};
	
	FNormalCoderSmith();	

	/** Encode a vector */
	FEncodedNormal Encode(const FVector& Value);

	/** Decode a vector */
	FVector Decode(const FEncodedNormal& Value);

	/** Convert from Cartesian vectors to spherical coordinates */
	static FORCEINLINE FSphericalCoordinates VectorToSpherical(const FVector& Vector);
	/** Convert from spherical coordinates to Cartesian vectors */
	static FORCEINLINE FVector SphericalToVector(const FSphericalCoordinates& Spherical);
private:

	/** Generate our table of NThetas. The lower the MaxError, the higher the values in the NTheta table, i.e., the number of bins */
	void GenerateNThetaTable(int32 NPhi, float MaxError, TArray<int32>& NTheta);
	/** Calculates a NTheta (number of azimuth bins) for a specific polar index */
	int32 CalcNTheta(int32 JIndex, float MaxError);
	/** The total amount of bins the sphere is split into, i.e., number of possible (J,K) combinations */
	int32 GetTotalBinCount() const;

	/** Number of bins for a given polar index */
	FORCEINLINE int32 GetNtheta(int32 IndexJ)
	{
		return NThetaTable[IndexJ];
	}

	/** Transforms a vector from its absolute position to its relative position to the previously-seen vector */
	FORCEINLINE FVector MoveFrameEncode(const FVector& Value);
	/** Transforms a relative vector back to its absolute position */
	FORCEINLINE FVector UnmoveFrame(const FVector& Moved, FMatrix& TransformMatrix);	
	/** Update our absolute-relative transformation matrix */
	FORCEINLINE void UpdateRotationMatrix(const FVector& UnmovedVector, FVector& ColumnX, FVector& ColumnY, FVector& ColumnZ);

	/** Always positive modulo operation helper */
	int32 PositiveModulo(int32 i, int32 n)
	{
		return (i % n + n) % n;
	}

	/** Helpers to get our column vectors in our matrix. These can probably be replaced by UnrealEngine-specific operations. */
	FORCEINLINE FVector GetMatrixColumn(FMatrix& Matrix, int32 Column);
	FORCEINLINE void SetMatrixColumn(FMatrix& Matrix, int32 Column, const FVector& Value);	
};

/** Testing functionality. */
class FNormalCoderSmithTest
{
public:
	static void Test();

private:
	static int32 Denormalize(float Value);
	static FIntVector Denormalize(const FVector& Value);
};
