// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"

/**
	Ring buffer with a given capacity, i.e., a fixed size stack that overwrites old values
	Push values at the top using Push(), retrieve values using Peek(depth), e.g., Peek(0) for the most recent, Peek(1) second most recent etc.
*/
template <class T, size_t N>
class FRingBuffer
{
public:
	FRingBuffer(uint32 Capacity) : Top(0), Count(0)
	{
		Buffer.AddDefaulted(Capacity);
	}

	FRingBuffer(uint32 Capacity, const T& DefaultValue) : Top(0), Count(0)
	{
		Buffer.Init(DefaultValue, Capacity);
	}

	/**
		Push an element to the top, Use Peek(0)	to retrieve it
	*/
	FORCEINLINE void Push(const T& value)
	{
		Count = (Count + 1) % Capacity();
		Top = (Top + Capacity() - 1) % Capacity(); // Front = Front - 1	
		Buffer[Top] = value;
	}
	
	/**
		Retrieve value indexed starting from the top, e.g., Peek(0) for most recent
	*/
	FORCEINLINE const T& Peek(uint32 Index) const
	{
		if (Index >= Capacity())
		{
			Index = Capacity() - 1;
		}
		return Buffer[(Top + Index) % Capacity()];
	}

	/**
		Retrieve value indexed starting from the top, e.g., Peek(0) for most recent
	*/
	FORCEINLINE const T& operator [](uint32 index) const
	{
		return Peek(index);
	}

	/**
		Capacity of the ring buffer
	*/
	FORCEINLINE uint32 Capacity() const
	{
		return Buffer.Num();
	}

	/**
		Number of the actual elements in the buffer
	*/
	FORCEINLINE uint32 Num() const
	{
		return Count;
	}

private:
	TArray<T, TInlineAllocator<N>> Buffer;
	uint32 Top;
	uint32 Count;
};
