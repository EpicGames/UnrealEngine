// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamingGeometryCacheData.h"
#include "Async/AsyncFileHandle.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheModule.h"
#include "StreamingGeometryCacheData.h"
#include "GeometryCacheStreamingManager.h"
#include "RenderingThread.h"
#include "ProfilingDebugging/IoStoreTrace.h"

static TAutoConsoleVariable<float> CVarPrefetchSeconds(
	TEXT("GeometryCache.PrefetchSeconds"),
	0.5,
	TEXT("The amount of data (expressed in seconds of animation) to preload of geometry caches. This is the data blockingly loaded at component spawn time."),
	ECVF_Scalability);

DECLARE_CYCLE_STAT(TEXT("Prefetch Data"), STAT_PrefetchData, STATGROUP_GeometryCache);
DECLARE_DWORD_COUNTER_STAT(TEXT("Outstanding Requests"), STAT_OutstandingRequests, STATGROUP_GeometryCache);
DECLARE_MEMORY_STAT(TEXT("Streamed Chunks"), STAT_ChunkDataStreamed, STATGROUP_GeometryCache);
DECLARE_MEMORY_STAT(TEXT("Resident Chunks"), STAT_ChunkDataResident, STATGROUP_GeometryCache);

FStreamingGeometryCacheData::FStreamingGeometryCacheData(UGeometryCacheTrackStreamable* InTrack)
	: Track(InTrack)
{
}

FStreamingGeometryCacheData::~FStreamingGeometryCacheData()
{
	check(IsInGameThread());

	// Flush the render thread so any decoding still happening is finished and thus no maps held by the render thread.
	FlushRenderingCommands();

	// Wait for all outstanding requests to finish
	BlockTillAllRequestsFinished();
	check(ChunksRequested.Num() == 0);

	// Free data associated with all chunks
	for (auto Iter = Chunks.CreateIterator(); Iter; ++Iter)
	{
		RemoveResidentChunk(Iter.Value());
	}
	Chunks.Empty();
}

void FStreamingGeometryCacheData::ResetNeededChunks()
{
	ChunksNeeded.Empty();
}

void FStreamingGeometryCacheData::AddNeededChunk(uint32 ChunkIndex)
{
	ChunksNeeded.AddUnique(ChunkIndex);
}

FResidentChunk& FStreamingGeometryCacheData::AddResidentChunk(int32 ChunkId, const FStreamedGeometryCacheChunk &ChunkInfo)
{
	FResidentChunk& result = Chunks.Add(ChunkId);
	result.Refcount = 0;
	result.Memory = nullptr;
	result.DataSize = ChunkInfo.DataSize;
	result.IORequest = nullptr;
	return result;
}

void FStreamingGeometryCacheData::RemoveResidentChunk(FResidentChunk& LoadedChunk)
{
	checkf(LoadedChunk.Refcount == 0, TEXT("Tried to remove a chunk wich was still mapped. Make sure there is an unmap for every map."));
	checkf(LoadedChunk.IORequest == nullptr, TEXT("RemoveResidentChunk was called on a chunk which hasn't been processed by ProcessCompletedChunks yet."));

	// Already loaded, so free it
	if (LoadedChunk.Memory != NULL)
	{
		DEC_MEMORY_STAT_BY(STAT_ChunkDataResident, LoadedChunk.DataSize);
		FMemory::Free(LoadedChunk.Memory);
	}

	LoadedChunk.Memory = nullptr;
	LoadedChunk.IORequest = nullptr;
	LoadedChunk.DataSize = 0;
	LoadedChunk.Refcount = 0;
}

/**
This is called from some random thread when reading is complete.
*/
void FStreamingGeometryCacheData::OnAsyncReadComplete(int32 LoadedChunkIndex, IBulkDataIORequest* ReadRequest)
{
	// We should do the least amount of work possible here as to not stall the async io threads.
	// We also cannot take the critical section here as this would lead to a deadlock between the
	// our critical section and the async-io internal critical section. 
	// So we just put this on queue here and then process the results later when we are on a different
	// thread that already holds our lock.
	// Game Thread:												... meanwhile on the Async loading thread:
	// - Get CriticalSection									- Get CachedFilesScopeLock as part of async code
	// - Call some async function								- Hey a request is complete start OnAsyncReadComplete
	// - Try get CachedFilesScopeLock as part of this function	- TRY get CriticalSection section waits for Game Thread
	// Both are waiting for each other's locks...
	// Note we cant clean the IO request up here. Trying to delete the request would deadlock
	// as delete waits until the request is complete bus it is only complete after the
	// callback returns ans since we're in the callback...

	CompletedChunks.Enqueue(FCompletedChunk(LoadedChunkIndex, ReadRequest));
}

/**
This does a blocking load for the first few seconds based on the component's current settings.
This ensures we got something to display initially.
*/
void FStreamingGeometryCacheData::PrefetchData(UGeometryCacheComponent *Component)
{
	SCOPE_CYCLE_COUNTER(STAT_PrefetchData)

	FScopeLock Lock(&CriticalSection);
	check(IsInGameThread());

	float RequestStartTime = Component->GetAnimationTime();

	// Blocking load half a second of data
	float RequestEndTime = RequestStartTime + Component->GetPlaybackDirection() * CVarPrefetchSeconds.GetValueOnGameThread();
	if (RequestStartTime > RequestEndTime)
	{
		float tmp = RequestEndTime;
		RequestEndTime = RequestStartTime;
		RequestStartTime = tmp;
	}

	TArray<int32> NewChunksNeeded;
	Track->GetChunksForTimeRange(RequestStartTime, RequestEndTime, Component->IsLooping(), NewChunksNeeded);

	for (int32 ChunkId : NewChunksNeeded)
	{
		ChunksNeeded.AddUnique(ChunkId);
	}

	for (int32 ChunkId : NewChunksNeeded)
	{
		// We just check here in case anything got loaded asynchronously last minute
		// to avoid unnecessary loading it synchronously again
		ProcessCompletedChunks();

		// Already got it
		if (ChunksAvailable.Contains(ChunkId))
		{
			continue;
		}

		// Still waiting for eviction, revive it
		if (ChunksEvicted.Contains(ChunkId))
		{
			ChunksEvicted.Remove(ChunkId);
			ChunksAvailable.Add(ChunkId);
			continue;
		}

		// Already requested an async load but not complete yet ... nothing much to do about this
		// it will just be loaded twice
		if (ChunksRequested.Contains(ChunkId))
		{
			ChunksRequested.Remove(ChunkId);
			// DEC_DWORD_STAT(STAT_OutstandingRequests); This is not needed here it will be DEC'ed when the async completes
		}

		// Load chunk from bulk data if available.
		FStreamedGeometryCacheChunk& Chunk = Track->GetChunk(ChunkId);
		check(Chunk.BulkData.GetBulkDataSize() > 0);
		check(Chunk.BulkData.GetBulkDataSize() == Chunk.DataSize);
		FResidentChunk &ResidentChunk = AddResidentChunk(ChunkId, Chunk);
		ResidentChunk.Memory = static_cast<uint8*>(FMemory::Malloc(Chunk.DataSize));
		INC_MEMORY_STAT_BY(STAT_ChunkDataResident, Chunk.DataSize);
		INC_MEMORY_STAT_BY(STAT_ChunkDataStreamed, Chunk.DataSize);
		Chunk.BulkData.GetCopy((void**)&ResidentChunk.Memory,true); //note: This does the actual loading internally...
		ChunksAvailable.Add(ChunkId);
	}
}

void FStreamingGeometryCacheData::UpdateStreamingStatus()
{
	FScopeLock Lock(&CriticalSection);

	// Find any chunks that aren't available yet
	for (int32 NeededIndex : ChunksNeeded)
	{
		if (!ChunksAvailable.Contains(NeededIndex))
		{
			// Revive it if it was still pinned for some other thread.
			if (ChunksEvicted.Contains(NeededIndex))
			{
				ChunksEvicted.Remove(NeededIndex);
				ChunksAvailable.Add(NeededIndex);
				continue;
			}

			// If not requested yet, then request a load
			if (!ChunksRequested.Contains(NeededIndex))
			{
				FStreamedGeometryCacheChunk& Chunk = Track->GetChunk(NeededIndex);

				// This can happen in the editor if the asset hasn't been saved yet.
				if (Chunk.BulkData.IsBulkDataLoaded())
				{
					FResidentChunk &ResidentChunk = AddResidentChunk(NeededIndex, Chunk);
					check(Chunk.BulkData.GetBulkDataSize() == Chunk.DataSize);
					ResidentChunk.Memory = FMemory::Malloc(Chunk.DataSize);
					INC_MEMORY_STAT_BY(STAT_ChunkDataResident, Chunk.DataSize);
					const void *P = Chunk.BulkData.LockReadOnly();
					FMemory::Memcpy(ResidentChunk.Memory, P, Chunk.DataSize);
					Chunk.BulkData.Unlock();
					ChunksAvailable.Add(NeededIndex);
					continue;
				}

				checkf(Chunk.BulkData.CanLoadFromDisk(), TEXT("Bulk data is not loaded and cannot be loaded from disk!"));
				check(!Chunk.BulkData.IsStoredCompressedOnDisk()); // We do not support compressed Bulkdata for this system

				FResidentChunk &ResidentChunk = AddResidentChunk(NeededIndex, Chunk);

				// Todo find something more smart...
				EAsyncIOPriorityAndFlags AsyncIOPriority = AIOP_BelowNormal;

				// Kick of a load
				check(Chunk.BulkData.GetBulkDataSize() == ResidentChunk.DataSize);

				FBulkDataIORequestCallBack AsyncFileCallBack = [this, NeededIndex](bool bWasCancelled, IBulkDataIORequest* Req)
				{
					this->OnAsyncReadComplete(NeededIndex, Req);
				};
				
				TRACE_IOSTORE_METADATA_SCOPE_TAG("GeometryCache");
				ResidentChunk.IORequest = Chunk.BulkData.CreateStreamingRequest(AsyncIOPriority, &AsyncFileCallBack, nullptr);
				if (!ResidentChunk.IORequest)
				{
					UE_LOGF(LogGeoCaStreaming, Error, "Geometry cache streaming read request failed.");
					return;
				}

				// Add it to the list
				ChunksRequested.Add(NeededIndex);
				INC_DWORD_STAT(STAT_OutstandingRequests);
			}

			// Nothing to do the chunk was requested and will be streamed in soon (hopefully)
		}
	}

	// Update bookkeeping with any recently completed chunks
	ProcessCompletedChunks();

	// Find indices that aren't needed anymore and add them to the list of chunks to evict
	for (int32 ChunkId = ChunksAvailable.Num() - 1; ChunkId >= 0; ChunkId--)
	{
		if (!ChunksNeeded.Contains(ChunksAvailable[ChunkId]))
		{
			ChunksEvicted.AddUnique(ChunksAvailable[ChunkId]);
			ChunksAvailable.RemoveAt(ChunkId);
		}
	}

	// Try to evict a bunch of chunks. Chunks which are still mapped (by other threads) can't be evicted
	// but others are free to go.
	for (int32 ChunkId = ChunksEvicted.Num() - 1; ChunkId >= 0; ChunkId--)
	{
		FResidentChunk &ResidentChunk = Chunks[ChunksEvicted[ChunkId]];
		if (ResidentChunk.Refcount == 0)
		{
			RemoveResidentChunk(ResidentChunk);
			ChunksEvicted.RemoveAt(ChunkId);
		}
	}

	// Delete the read requests that were delayed
	TSet<IBulkDataIORequest*> ReadyForDeletion;
	for (IBulkDataIORequest* ReadRequest : DelayedDeleteReadRequests)
	{
		if (ReadRequest->PollCompletion())
		{
			ReadyForDeletion.Add(ReadRequest);
		}
	}

	for (IBulkDataIORequest* ReadRequest : ReadyForDeletion)
	{
		DelayedDeleteReadRequests.Remove(ReadRequest);
		delete ReadRequest;
	}
}

bool FStreamingGeometryCacheData::BlockTillAllRequestsFinished(float TimeLimit)
{
	QUICK_SCOPE_CYCLE_COUNTER(FGeoCaStreaming_BlockTillAllRequestsFinished);
	FScopeLock Lock(&CriticalSection);

	if (TimeLimit == 0.0f)
	{
		for (auto Iter = Chunks.CreateIterator(); Iter; ++Iter)
		{
			if (Iter.Value().IORequest)
			{
				Iter.Value().IORequest->WaitCompletion();
				ProcessCompletedChunks();
			}
		}
	}
	else
	{
		double EndTime = FPlatformTime::Seconds() + TimeLimit;
		for (auto Iter = Chunks.CreateIterator(); Iter; ++Iter)
		{
			if (Iter.Value().IORequest)
			{
				float ThisTimeLimit = static_cast<float>(EndTime - FPlatformTime::Seconds());
				if (ThisTimeLimit < .001f || // one ms is the granularity of the platform event system
					!Iter.Value().IORequest->WaitCompletion(ThisTimeLimit))
				{
					return false;
				}
				ProcessCompletedChunks();
			}
		}
	}

	return true;
}

void FStreamingGeometryCacheData::ProcessCompletedChunks()
{
	//Note: This function should only be called from code which owns the CriticalSection
	if (!IsInGameThread() && !IsInRenderingThread())
	{
		return;
	}

	FCompletedChunk CompletedChunk;
	while (CompletedChunks.Dequeue(CompletedChunk))
	{
		FResidentChunk* Chunk = Chunks.Find(CompletedChunk.LoadedChunkIndex);
		if (!Chunk)
		{
			UE_LOGF(LogGeoCaStreaming, Error, "Got a stray async read request");
			return;
		}

		// Chunks can be queued up multiple times when scrubbing but we can trust the LoadedChunkIndex of the completed chunk
		// so all we need to check is if the FResidentChunk  has a valid IORequest pointer or not.
		// If it is nullptr then we already processed a request for the chunk and it can be ignored.
		if (Chunk->IORequest != nullptr)
		{
			// Check to see if we successfully managed to load anything
			uint8* Mem = CompletedChunk.ReadRequest->GetReadResults();
			if (Mem)
			{
				Chunk->Memory = Mem;
				ChunksAvailable.Add(CompletedChunk.LoadedChunkIndex);
				ChunksRequested.Remove(CompletedChunk.LoadedChunkIndex);
				DEC_DWORD_STAT(STAT_OutstandingRequests);
				INC_MEMORY_STAT_BY(STAT_ChunkDataResident, Chunk->DataSize);
				INC_MEMORY_STAT_BY(STAT_ChunkDataStreamed, Chunk->DataSize);
				IGeometryCacheStreamingManager::Get().IoBandwidth.Add(Chunk->DataSize);
			}
			else
			{
				UE_LOGF(LogGeoCaStreaming, Error, "Async loading request failed!");
				ChunksRequested.Remove(CompletedChunk.LoadedChunkIndex);
				// Fix me do we want to recover from this? Granite simply reschedules requests
				// as it may have failed for transient reasons (buffer contention, ...)
			}

			Chunk->IORequest = nullptr;
		}

		// Clean up the now fully processed IO request
		if (CompletedChunk.ReadRequest->PollCompletion())
		{
			delete CompletedChunk.ReadRequest;
		}
		else
		{
			// The ReadRequest SetComplete is a 2-step process where the read completion callback OnAsyncReadComplete is called first,
			// then the flag checked by PollCompletion, bCompleteAndCallbackCalled, is set to true.
			// Since the callback is called from a worker thread, there's a possible window where the CompletedChunk gets processed
			// but its ReadRequest completion flag is still false. In that case, the deletion is queued to be processed later
			DelayedDeleteReadRequests.Add(CompletedChunk.ReadRequest);
		}
	}
}

const uint8* FStreamingGeometryCacheData::MapChunk(uint32 ChunkIndex, uint32* OutChunkSize)
{
	FScopeLock Lock(&CriticalSection);

	// Quickly check before mapping if maybe something new arrived we haven't done bookkeeping for yet
	ProcessCompletedChunks();

	if (!ChunksAvailable.Contains(ChunkIndex))
	{
		if (!ChunksRequested.Contains(ChunkIndex))
		{
			if (ChunksEvicted.Contains(ChunkIndex))
			{
				UE_LOGF(LogGeoCaStreaming, Log, "Tried to map an evicted chunk: %i.", ChunkIndex);
			}
			else
			{
				UE_LOGF(LogGeoCaStreaming, Log, "Tried to map an unavailabe non-requested chunk: %i.", ChunkIndex);
			}
		}
		else
		{
			UE_LOGF(LogGeoCaStreaming, Log, "Tried to map a chunk (%i) that is still being streamed in.", ChunkIndex);
		}
		return nullptr;
	}
	else
	{
		FResidentChunk *ResidentChunk = Chunks.Find(ChunkIndex);
		check(ResidentChunk);
		if (OutChunkSize)
		{
			*OutChunkSize = ResidentChunk->DataSize;
		}
		ResidentChunk->Refcount++;
		return (uint8*)ResidentChunk->Memory;
	}
}

void FStreamingGeometryCacheData::UnmapChunk(uint32 ChunkIndex)
{
	FScopeLock Lock(&CriticalSection);

	FResidentChunk* ResidentChunk = Chunks.Find(ChunkIndex);

	if (ResidentChunk != nullptr )
	{
		checkf(ResidentChunk->Refcount > 0, TEXT("Map/Unmap out of balance. Make sure you unmap once fore every map."));
		checkf(ChunksAvailable.Contains(ChunkIndex) || ChunksEvicted.Contains(ChunkIndex), TEXT("Tried to unmap a chunk in an invalid state."));
		ResidentChunk->Refcount--;
	}
	else
	{
		UE_LOGF(LogGeoCaStreaming, Log, "Tried to unmap an unknown chunk.");
	}
}

bool FStreamingGeometryCacheData::IsStreamingInProgress()
{
	FScopeLock Lock(&CriticalSection);
	return ChunksRequested.Num() > 0;
}
