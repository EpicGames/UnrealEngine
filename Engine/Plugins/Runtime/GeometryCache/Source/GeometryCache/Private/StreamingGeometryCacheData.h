// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Containers/Queue.h"

class IAsyncReadRequest;
class IBulkDataIORequest;
class UGeometryCacheTrackStreamable;
class UGeometryCacheComponent;
struct FStreamedGeometryCacheChunk;

/**
An actual chunk resident in memory.
*/
struct FResidentChunk
{
	void* Memory;
	int32 DataSize;
	int32 Refcount;
	IBulkDataIORequest* IORequest; //null when resident, nonnull when being loaded
};

/**
	The results of a completed async-io request.
*/
struct FCompletedChunk
{
	IBulkDataIORequest* ReadRequest;
	int32 LoadedChunkIndex;

	FCompletedChunk() : ReadRequest(nullptr), LoadedChunkIndex(0) {}
	FCompletedChunk(int32 SetLoadedChunkIndex, IBulkDataIORequest* SetReadRequest) : ReadRequest(SetReadRequest), LoadedChunkIndex(SetLoadedChunkIndex) {}
};

/**
For every UGeometryCache one of these is created for by the streaming manager.
This keeps this functionality separate from the main UGeometryCacheTrackStreamable and tied to the manager.
See it as a "component" on the UGeometryCacheTrackStreamable
*/
class FStreamingGeometryCacheData
{
public:
	FStreamingGeometryCacheData(UGeometryCacheTrackStreamable* Track);
	~FStreamingGeometryCacheData();

	void UpdateStreamingStatus();

	const uint8* MapChunk(uint32 ChunkIndex, uint32* OutChunkSize = nullptr);
	void UnmapChunk(uint32 ChunkIndex);

	bool IsStreamingInProgress();
	bool BlockTillAllRequestsFinished(float TimeLimit = 0.0f);
	void PrefetchData(UGeometryCacheComponent* Component);

	void ResetNeededChunks();
	void AddNeededChunk(uint32 ChunkIndex);

private:

	FResidentChunk &AddResidentChunk(int32 ChunkId, const FStreamedGeometryCacheChunk &ChunkInfo);
	void RemoveResidentChunk(FResidentChunk& LoadedChunk);
	void OnAsyncReadComplete(int32 LoadedChunkIndex, IBulkDataIORequest* ReadRequest);
	void ProcessCompletedChunks();

	// The track we are associated with
	UGeometryCacheTrackStreamable *Track;

	// Chunks that ideally would be loaded at this point in time
	// There may be more or less actual chunks loaded (more = cached chunks, less = we're still waiting for the disc)
	// This should only be used from the main thread. It can be modified without taking the lock. Changes are then
	// "latched" to other data structures/threads in the UpdateStreamingStatus function.
	TArray<int32> ChunksNeeded;

	// List of chunks currently resident in memory
	TArray<int32> ChunksAvailable;

	// This this does not necessary contains only chunks in the ChunksAvailable lists
	// for example chunks in the ChunksRequested will also be in here.
	TMap<int32, FResidentChunk> Chunks;

	// Chunks requested to be streamed in but not available yet
	TArray<int32> ChunksRequested;

	// Chunks to be evicted. Chunks may linger here for a while
	// until they are fully unpinned
	TArray<int32> ChunksEvicted;

	TArray<IAsyncReadRequest*>IoRequestsToCompleteAndDeleteOnGameThread;

	// Chunks that have finished loading but have not finished their post-load bookkeping
	// they are still not part of the ChunksAvailable list.
	TQueue<FCompletedChunk, EQueueMode::Mpsc> CompletedChunks;

	TSet<IBulkDataIORequest*> DelayedDeleteReadRequests;

	mutable FCriticalSection CriticalSection;
};

