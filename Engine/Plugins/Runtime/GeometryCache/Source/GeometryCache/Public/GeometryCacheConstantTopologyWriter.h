// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"
#include "UObject/StrongObjectPtr.h"


struct FGeometryCacheMeshBatchInfo;
struct FPackedNormal;
class FSkeletalMeshLODModel;
class UGeometryCache;
class UGeometryCacheTrack;
class UMaterialInterface; 
class USkinnedAsset;
class UStaticMesh;

namespace UE::GeometryCacheHelpers
{
#if WITH_EDITOR
	/**
	 * Helper class to write a GeometryCache asset.
	 * Usage:
	 * 	FGeometryCacheConstantTopologyWriter Writer(MyCache);
	 * 	Writer.AddMaterials(...);
	 * 	FGeometryCacheConstantTopologyWriter::FTrackWriter& TrackWriter = Writer.AddGetTrackWriter(); // First track
	 * 	TrackWriter.Indices = ...;
	 * 	TrackWriter.UVs = ...;
	 *  ...
	 * 	TrackWriter.WriteAndClose(PositionsToMoveFrom);
	 * 	FGeometryCacheConstantTopologyWriter::FTrackWriter& TrackWriter = Writer.AddGetTrackWriter(); // Second track
	 * 	...
	 */
	class FGeometryCacheConstantTopologyWriter
	{
	public:
		struct FConfig
		{
			float FPS = 30.0f;
			float PositionPrecision = 0.001f;
			uint32 TextureCoordinatesNumberOfBits = 10;
		};
		
		/**
		 * Construct a new FGeometryCacheConstantTopologyWriter object. This will remove all existing tracks from the cache.
		 * @param OutCache 
		 */
		GEOMETRYCACHE_API FGeometryCacheConstantTopologyWriter(UGeometryCache& OutCache);
		GEOMETRYCACHE_API FGeometryCacheConstantTopologyWriter(UGeometryCache& OutCache, const FConfig& Config);
		GEOMETRYCACHE_API ~FGeometryCacheConstantTopologyWriter();

		FGeometryCacheConstantTopologyWriter(const FGeometryCacheConstantTopologyWriter&) = delete;
		FGeometryCacheConstantTopologyWriter& operator=(const FGeometryCacheConstantTopologyWriter&) = delete;

		struct FFrameData
		{
			TArray<FVector3f> Positions;
			TArray<FVector3f> Normals;
			TArray<FVector3f> TangentsX;
		};

		struct FVisibilitySample
		{
			int32 FrameIndex = 0;
			bool bVisible = true;
		};

		struct FTrackWriter
		{
			GEOMETRYCACHE_API FTrackWriter(FGeometryCacheConstantTopologyWriter& InOwner, FName TrackName = FName());
			GEOMETRYCACHE_API ~FTrackWriter();

			FTrackWriter(const FTrackWriter&) = delete;
			FTrackWriter& operator=(const FTrackWriter&) = delete;

			TArray<uint32> Indices;
			TArray<FVector2f> UVs;
			TArray<FColor> Colors;
			TArray<uint32> ImportedVertexNumbers;
			TArray<FGeometryCacheMeshBatchInfo> BatchesInfo;
			TOptional<TArray<int32>> SourceVertexIndices;

			/**
			 * Move the position data to the cache track and close the TrackWriter. 
			 * Once closed, the track will be added to the geometry cache and the TrackWriter cannot be used anymore.
			 * @param PositionsToMoveFrom Array of positions to move from. 
			 * The size of the array equals to the number of frames. 
			 * The size of each array element equals to the number of vertices. 
			 * The number of vertices must be the same for all frames.
			 * @return true if successfully write data and close the track writer.
			 */
			GEOMETRYCACHE_API bool WriteAndClose(TArrayView<TArray<FVector3f>> PositionsToMoveFrom);

			/**
			 * Move the frame data to the cache track and close the TrackWriter.
			 * Similar to WriteAndClose(TArrayView<TArray<FVector3f>> PositionsToMoveFrom), but also supports normals and tangents.
			 * Normals and tangents are optional. If they are not provided, the track will compute them.
			 * Normals and tangents must have the same size as positions.
			 */
			GEOMETRYCACHE_API bool WriteAndClose(TArrayView<FFrameData> FramesToMoveFrom);

			/**
			 * Move the frame data to the cache track and close the TrackWriter.
			 * Similar to WriteAndClose(TArrayView<FFrameData> FramesToMoveFrom), but also supports visibility samples.
			 * Each visibility sample defines the visibility of the track starting from the specified frame to the next visibility sample
			 */
			GEOMETRYCACHE_API bool WriteAndClose(TArrayView<FFrameData> FramesToMoveFrom, const TArray<FVisibilitySample>& VisibilitySamples);
			
		private:
			TStrongObjectPtr<UGeometryCacheTrack> Track;
			FGeometryCacheConstantTopologyWriter* Owner = nullptr;
		};

		GEOMETRYCACHE_API FTrackWriter& AddTrackWriter(FName TrackName = FName());
		GEOMETRYCACHE_API FTrackWriter& GetTrackWriter(int32 Index);
		GEOMETRYCACHE_API int32 GetNumTracks() const;
		GEOMETRYCACHE_API void AddMaterials(const TArray<TObjectPtr<UMaterialInterface>>& Materials);
		GEOMETRYCACHE_API int32 GetNumMaterials() const;

	private:
		TStrongObjectPtr<UGeometryCache> Cache;
		TArray<FTrackWriter> TrackWriters;
		FConfig Config;
	};

	/**
	 * @brief This will create a track writer and fill in the track writer's data (indices, UVs, materials .etc) from the skinned asset.
	 * This only creates one track for the skinned asset. If you want to create multiple tracks based on imported mesh info, checkout AddTrackWriterFromSkinnedAsset() in ChaosClothGenerator.cpp. 
	 * Usage:
	 * 	FGeometryCacheConstantTopologyWriter Writer(MyCache);
	 * 	int32 Index = AddTrackWriterFromSkinnedAsset(Writer, Asset);
	 * 	if (Index != INDEX_NONE)
	 * 	{
	 * 		Writer.GetTrackWriter(Index).WriteAndClose(PositionsToMoveFrom);
	 * 	}
	 */
	GEOMETRYCACHE_API int32 AddTrackWriterFromSkinnedAsset(FGeometryCacheConstantTopologyWriter& Writer, const USkinnedAsset& Asset);

	/**
	 * @brief This will create a track writer and fill in the track writer's data (indices, UVs, .etc) from the skinned asset, with the option to specify a set of materials 
	 * which typically come from a specific skeletal mesh component using this skinned asset.
	 */
	GEOMETRYCACHE_API int32 AddTrackWriterFromSkinnedAssetAndMaterials(FGeometryCacheConstantTopologyWriter& Writer, const USkinnedAsset& Asset, int32 LODIndex, const TArray<TObjectPtr<UMaterialInterface>>& Materials);

	/**
	 * @brief This will create a track writer and fill in the track writer's data (indices, UVs, materials .etc) from the static mesh.
	 * This only creates one track for the static mesh. If you want to create multiple tracks based on imported mesh info, checkout AddTrackWriterFromSkinnedAsset() in ChaosClothGenerator.cpp.
	 * Usage:
	 * @code
	 * 	FGeometryCacheConstantTopologyWriter Writer(MyCache);
	 * 	int32 Index = AddTrackWriterFromStaticMesh(Writer, Asset);
	 * 	if (Index != INDEX_NONE)
	 * 	{
	 * 		Writer.GetTrackWriter(Index).WriteAndClose(PositionsToMoveFrom);
	 * 	}
	 * @endcode
	 */
	GEOMETRYCACHE_API int32 AddTrackWriterFromStaticMesh(FGeometryCacheConstantTopologyWriter& Writer, const UStaticMesh& Asset);

	/**
	 * @brief This will create a track writer and fill in the track writer's data (indices, UVs, .etc) from the static mesh, with the option to specify a set of materials
	 */
	GEOMETRYCACHE_API int32 AddTrackWriterFromStaticMeshAndMaterials(
		FGeometryCacheConstantTopologyWriter& Writer,
		const UStaticMesh& StaticMesh,
		int32 LODIndex,
		const TArray<TObjectPtr<UMaterialInterface>>& Materials);

	/**
	 * @brief This will create multiple track writers and fill in the track writer's data (indices, UVs, materials .etc) from the template geometry cache. 
	 * The number of track writers created equals to the number of tracks in the template geometry cache. 
	 * @return The number of track writers created.
	 */
	GEOMETRYCACHE_API int32 AddTrackWritersFromTemplateCache(FGeometryCacheConstantTopologyWriter& Writer, const UGeometryCache& TemplateCache);
#endif // WITH_EDITOR
};