// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathUtility.h"

struct GeometyCacheHelpers 
{
	/**
		Use this instead of fmod when working with looping animations as fmod gives incorrect results when using negative times.
	*/
	static inline float WrapAnimationTime(float Time, float Duration)
	{
		return Time - Duration * FMath::FloorToFloat(Time / Duration);
	}
};

namespace MeshAttribute::VertexInstance
{
	extern GEOMETRYCACHE_API const FName Velocity;
}

struct FGeometryCacheMeshData;
struct FMeshDescription;

namespace UE::GeometryCache::Utils
{
	struct FMeshDataConversionArguments
	{	
		int32 MaterialOffset = 0;
		float FramesPerSecond = 24.0f;
		bool bUseVelocitiesAsMotionVectors = true;
		bool bStoreImportedVertexNumbers = false;
	};

	GEOMETRYCACHE_API void GetGeometryCacheMeshDataFromMeshDescription(FGeometryCacheMeshData& OutMeshData, FMeshDescription& MeshDescription, const FMeshDataConversionArguments& Args);
}