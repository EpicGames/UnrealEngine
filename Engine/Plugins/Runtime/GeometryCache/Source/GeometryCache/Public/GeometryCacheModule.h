// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"

LLM_DECLARE_TAG_API(GeometryCache, GEOMETRYCACHE_API);

/**
 * The public interface to this module
 */
class FGeometryCacheModule : public IModuleInterface
{
public:
	FGeometryCacheModule() {}

	virtual void StartupModule();
	virtual void ShutdownModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FGeometryCacheModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FGeometryCacheModule >("GeometryCache");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "GeometryCache" );
	}
private:
};

DECLARE_STATS_GROUP(TEXT("GeometryCache"), STATGROUP_GeometryCache, STATCAT_Advanced);
