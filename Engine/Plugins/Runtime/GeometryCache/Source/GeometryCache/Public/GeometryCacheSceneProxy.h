// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialRelevance.h"
#include "RayTracingGeometry.h"
#include "PrimitiveSceneProxy.h"
#include "Materials/MaterialInterface.h"
#include "GeometryCacheVertexFactory.h"
#include "DynamicMeshBuilder.h"

#define UE_API GEOMETRYCACHE_API

class FDynamicPrimitiveUniformBuffer;
class FHitProxyId;

class FMeshElementCollector;
struct FGeometryCacheMeshData;
class FGeometryCacheTrackStreamableRenderResource;
struct FVisibilitySample;
class UGeometryCacheTrack;

/** Resource array to pass  */
class FGeomCacheVertexBuffer : public FVertexBuffer
{
public:

	void Init(int32 InSizeInBytes)
	{
		check(this->IsInitialized() == false);
		SizeInBytes = InSizeInBytes;
	}

	/* Create on rhi thread. Uninitialized with NumVertices space.*/
	UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	UE_API virtual void ReleaseRHI() override;

	/**
	 * Sugar function to update from a typed array.
	 */
	template<class DataType> void Update(FRHICommandListBase& RHICmdList, const TArray<DataType>& Vertices)
	{
		int32 InSize = Vertices.Num() * sizeof(DataType);
		UpdateRaw(RHICmdList, Vertices.GetData(), InSize, 1, 1);
	}

	void UpdatePositionsOnly(FRHICommandListBase& RHICmdList, const TArray<FDynamicMeshVertex>& Vertices)
	{
		const uint32 PositionOffset = STRUCT_OFFSET(FDynamicMeshVertex, Position);
		const uint32 PositionSize = sizeof(((FDynamicMeshVertex*)nullptr)->Position);
		UpdateRaw(RHICmdList, Vertices.GetData() + PositionOffset, Vertices.Num(), PositionSize, sizeof(FDynamicMeshVertex));
	}

	void UpdateExceptPositions(FRHICommandListBase& RHICmdList, const TArray<FDynamicMeshVertex>& Vertices)
	{
		const uint32 PositionSize = sizeof(((FDynamicMeshVertex*)nullptr)->Position);
		const uint32 PositionOffset = STRUCT_OFFSET(FDynamicMeshVertex, Position);

		static_assert(PositionOffset == 0, "Expecting position to be the first struct member");
		static_assert(PositionSize == STRUCT_OFFSET(FDynamicMeshVertex, TextureCoordinate), "Expecting the texture coordinate to immediately follow the Position");

		UpdateRaw(RHICmdList, (int8*)Vertices.GetData() + PositionSize, Vertices.Num(), sizeof(FDynamicMeshVertex) - PositionSize, sizeof(FDynamicMeshVertex));
	}

	/**
	 * Update the raw contents of the buffer, possibly reallocate if needed.
	 */
	UE_API void UpdateRaw(FRHICommandListBase& RHICmdList, const void* Data, int32 NumItems, int32 ItemSizeBytes, int32 ItemStrideBytes);

	/**
	 * Resize the buffer but don't initialize it with any data.
	 */
	UE_API void UpdateSize(FRHICommandListBase& RHICmdList, int32 NewSizeInBytes);

	/**
	* Resize the buffer but don't initialize it with any data.
	*/
	template<class DataType> void UpdateSizeTyped(FRHICommandListBase& RHICmdList, int32 NewSizeInElements)
	{
		UpdateSize(RHICmdList, sizeof(DataType) * NewSizeInElements);
	}

	/**
	 * Get the current size of the buffer
	 */
	unsigned GetSizeInBytes() { return SizeInBytes; }

	virtual FString GetFriendlyName() const override { return TEXT("FGeomCacheVertexBuffer"); }

	FRHIShaderResourceView* GetBufferSRV() const { return BufferSRV; }

protected:
	int32 SizeInBytes;
	FShaderResourceViewRHIRef BufferSRV;
};

class FGeomCacheTangentBuffer : public FGeomCacheVertexBuffer
{
public:
	UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

class FGeomCacheColorBuffer : public FGeomCacheVertexBuffer
{
public:
	UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/** Index Buffer */
class FGeomCacheIndexBuffer : public FIndexBuffer
{
public:
	int32 NumAllocatedIndices = 0; // Total allocated GPU index buffer size in elements
	int32 NumValidIndices = 0; // Current valid data region of the index buffer (may be smaller than allocated buffer)

	/* Create on rhi thread. Uninitialized with NumIndices space.*/
	UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	UE_API virtual void ReleaseRHI() override;

	/**
		Update the data and possibly reallocate if needed.
	*/
	UE_API void Update(FRHICommandListBase& RHICmdList, const TArray<uint32>& Indices);
	UE_API void UpdateSizeOnly(FRHICommandListBase& RHICmdList, int32 NewNumIndices);

	unsigned SizeInBytes() { return NumAllocatedIndices * sizeof(uint32); }

	FRHIShaderResourceView* GetBufferSRV() const { return BufferSRV; }

protected:
	FShaderResourceViewRHIRef BufferSRV;
};

/** Vertex Factory */
class FGeomCacheVertexFactory : public FGeometryCacheVertexVertexFactory
{
public:
	UE_API FGeomCacheVertexFactory(ERHIFeatureLevel::Type InFeatureLevel);

	UE_DEPRECATED(5.7, "Use InitComponentVF to setup the FDataType and then call SetData on the vertex factory with the command list")
	UE_API void Init(FRHICommandListBase& RHICmdList, const FVertexBuffer* PositionBuffer, const FVertexBuffer* MotionBlurDataBuffer, const FVertexBuffer* TangentXBuffer, const FVertexBuffer* TangentZBuffer, const FVertexBuffer* TextureCoordinateBuffer, const FVertexBuffer* ColorBuffer);

	UE_API static void InitDummyComponentVF(FGeometryCacheVertexVertexFactory::FDataType& OutData);	
	UE_API static void InitComponentVF(const FVertexBuffer* PositionBuffer, const FVertexBuffer* MotionBlurDataBuffer, const FVertexBuffer* TangentXBuffer, const FVertexBuffer* TangentZBuffer, const FVertexBuffer* TextureCoordinateBuffer, const FVertexBuffer* ColorBuffer, FGeometryCacheVertexVertexFactory::FDataType& OutData);	
};

// Hacky base class to avoid 8 bytes of padding after the vtable
class FGeomCacheTrackProxyFixLayout
{
public:
	virtual ~FGeomCacheTrackProxyFixLayout() = default;
};

/**
 * This the track proxy has some "double double buffering" going on.
 * First we keep two mesh frames. The one just before the current time and the one just after the current time. This is the full mesh and
 * we interpolate between it to derive the actual mesh for the exact time we're at.
 * Secondly we have two position buffers. The one for the current rendered frame and the one from the previous rendered frame (this is not the same as
 * the mesh frame, the mesh may be at say 10 fps then get interpolated to 60 fps rendered frames)
 */
class FGeomCacheTrackProxy : FGeomCacheTrackProxyFixLayout
{
public:

	FGeomCacheTrackProxy(ERHIFeatureLevel::Type InFeatureLevel)
		: VertexFactory(InFeatureLevel/*, "FGeomCacheTrackProxy"*/)
	{}

	virtual ~FGeomCacheTrackProxy() = default;

	/**
	 * Update the SampleIndex and MeshData for a given time
	 *
	 * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation is being played in a loop
	 * @param InOutMeshSampleIndex - Hold the MeshSampleIndex and will be updated if changed according to the Elapsed Time
	 * @param OutMeshData - Will hold the new MeshData if the SampleIndex changed
	 * @return true if the SampleIndex and MeshData were updated
	 */
	UE_API virtual bool UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData);

	/**
	 * Get the MeshData for a given SampleIndex
	 *
	 * @param SampleIndex - The sample index at which to retrieve the MeshData
	 * @param OutMeshData - Will hold the MeshData if it could be retrieved
	 * @return true if the MeshData was retrieved successfully
	 */
	UE_API virtual bool GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData);

	/**
	 * Check if the topology of two given SampleIndexes are compatible (ie. same topology)
	 *
	 * @param SampleIndexA - The first sample index to compare the topology
	 * @param SampleIndexB - The second sample index to compare the topology
	 * @return true if the topology is the same
	 */
	UE_API virtual bool IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB);

	/**
	 * Get the VisibilitySample for a given time
	 *
	 * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation is being played in a loop
	 * @return the VisibilitySample that corresponds to the given time
	 */
	UE_API virtual const FVisibilitySample& GetVisibilitySample(float Time, const bool bLooping) const;

	/**
	 * Find the two frames closest to the given time
	 * InterpolationFactor gives the position of the requested time slot between the two returned frames.
	 * 0.0 => We are very close to OutFrameIndex
	 * 1.0 => We are very close to OutNextFrameIndex
	 * If bIsPlayingBackwards it will return exactly the same indexes but in the reversed order. The
	 * InterpolationFactor will also be updated accordingly
	 *
	 * @param Time - (Elapsed)Time to check against
	 * @param bLooping - Whether or not the animation is being played in a loop
	 * @param bIsPlayingBackwards - Whether the animation is playing backwards or forwards
	 * @param OutFrameIndex - The closest frame index that corresponds to the given time
	 * @param OutNextFrameIndex - The frame index that follows OutFrameIndex
	 * @param InterpolationFactor - The interpolation value between the two frame times
	 */
	UE_API virtual void FindSampleIndexesFromTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32& OutFrameIndex, int32& OutNextFrameIndex, float& InInterpolationFactor);

	/**
	 * Initialize the render resources. Must be called before the render resources are used.
	 *
	 * @param NumVertices - The initial number of vertices to initialize the buffers with. Must be greater than 0
	 * @param NumIndices - The initial number of indices to initialize the buffers with. Must be greater than 0
	 */
	UE_API virtual void InitRenderResources(FRHICommandListBase& RHICmdList, int32 NumVertices, int32 NumIndices);

	/** MeshData storing information used for rendering this Track */
	FGeometryCacheMeshData* MeshData;
	FGeometryCacheMeshData* NextFrameMeshData;

	/** Frame numbers corresponding to MeshData, NextFrameMeshData */
	int32 FrameIndex;
	int32 NextFrameIndex;
	int32 PreviousFrameIndex;
	float InterpolationFactor;
	float PreviousInterpolationFactor;
	float SubframeInterpolationFactor;

	/** Material applied to this Track */
	TArray<UMaterialInterface*> Materials;

	/** Vertex buffers for this Track. There are two position buffers which we double buffer between, current frame and last frame*/
	FGeomCacheVertexBuffer PositionBuffers[2];
	uint32 PositionBufferFrameIndices[2]; // Frame indexes of the positions in the position buffer 
	float PositionBufferFrameTimes[2]; // Exact time after interpolation of the positions in the position buffer.
	uint32 CurrentPositionBufferIndex; // CurrentPositionBufferIndex%2  is the last updated position buffer

	int32 UploadedSampleIndex;

	FGeomCacheTangentBuffer TangentXBuffer;
	FGeomCacheTangentBuffer TangentZBuffer;
	FGeomCacheVertexBuffer TextureCoordinatesBuffer;
	FGeomCacheColorBuffer ColorBuffer;

	/** Index buffer for this Track */
	FGeomCacheIndexBuffer IndexBuffer;

	/** Vertex factory for this Track */
	FGeomCacheVertexFactory VertexFactory;

	/** The GeometryCacheTrack to which the proxy is associated */
	UGeometryCacheTrack* Track;

	/** World Matrix for this Track */
	FMatrix WorldMatrix;

	/** Flag to indicate which frame mesh data was selected during the update */
	bool bNextFrameMeshDataSelected;

	bool bResourcesInitialized;

#if RHI_RAYTRACING
	bool bInitializedRayTracing = false;
	FRayTracingGeometry RayTracingGeometry;
#endif
};

/** Procedural mesh scene proxy */
class FGeometryCacheSceneProxy : public FPrimitiveSceneProxy
{
public:
	UE_API SIZE_T GetTypeHash() const override;

	UE_API FGeometryCacheSceneProxy(class UGeometryCacheComponent* Component);
	UE_API FGeometryCacheSceneProxy(class UGeometryCacheComponent* Component, TFunction<FGeomCacheTrackProxy*()> TrackProxyCreator);

	UE_API virtual ~FGeometryCacheSceneProxy();

	// Begin FPrimitiveSceneProxy interface.
#if WITH_EDITOR
	UE_API virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif
	UE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	UE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	UE_API virtual bool CanBeOccluded() const override;
	UE_API virtual bool IsUsingDistanceCullFade() const override;
	UE_API virtual uint32 GetMemoryFootprint(void) const;
	UE_API uint32 GetAllocatedSize(void) const;
	// End FPrimitiveSceneProxy interface.

	UE_API void UpdateAnimation(FRHICommandListBase& RHICmdList, float NewTime, bool bLooping, bool bIsPlayingBackwards, float PlaybackSpeed, float MotionVectorScale, bool bInterpolateFrames = true);

	struct FAnimationUpdateParameters
	{
		float Time;
		bool bLooping;
		bool bIsPlayingBackwards;
		float PlaybackSpeed;
		float MotionVectorScale;
		bool bInterpolateFrames;
	};
	UE_API void UpdateAnimation(FRHICommandListBase& RHICmdList, const FAnimationUpdateParameters& UpdatedValues);

	/** Update world matrix for specific section */
	UE_API void UpdateSectionWorldMatrix(const int32 SectionIndex, const FMatrix& WorldMatrix);
	/** Update vertex buffer for specific section */
	UE_API void UpdateSectionVertexBuffer(const int32 SectionIndex, FGeometryCacheMeshData* MeshData);
	/** Update index buffer for specific section */
	UE_API void UpdateSectionIndexBuffer(const int32 SectionIndex, const TArray<uint32>& Indices);

	/** Clears the Sections array*/
	UE_API void ClearSections();

#if RHI_RAYTRACING
	UE_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override final;
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool HasRayTracingRepresentation() const override { return true; }
#endif

	const TArray<FGeomCacheTrackProxy*>& GetTracks() const { return Tracks; }

private:
	UE_API void FrameUpdate(FRHICommandListBase& RHICmdList) const;

#if RHI_RAYTRACING
	UE_API void InitRayTracing(FRHICommandListBase& RHICmdList);
#endif

	UE_API void CreateMeshBatch(
		FRHICommandListBase& RHICmdList,
		const FGeomCacheTrackProxy* TrackProxy,
		const struct FGeometryCacheMeshBatchInfo& BatchInfo,
		class FGeometryCacheVertexFactoryUserDataWrapper& UserDataWrapper,
		FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer,
		FMeshBatch& Mesh) const;

private:
	/** Array of Track Proxies */
	TArray<FGeomCacheTrackProxy*> Tracks;

#if WITH_EDITOR
	TArray<FHitProxyId> HitProxyIds;
#endif

	/** Scratch memory for frame update - do not use directly. */
	struct FScratchMemory
	{
		TArray<FVector3f> InterpolatedPositions;
		TArray<FPackedNormal> InterpolatedTangentX;
		TArray<FPackedNormal> InterpolatedTangentZ;
		TArray<FVector2f> InterpolatedUVs;
		TArray<FColor> InterpolatedColors;
		TArray<FVector3f> InterpolatedMotionVectors;

		void Prepare(int32 NumVertices, bool bHasMotionVectors)
		{
			// Clear entries but keep allocations.
			InterpolatedPositions.Reset();
			InterpolatedTangentX.Reset();
			InterpolatedTangentZ.Reset();
			InterpolatedUVs.Reset();
			InterpolatedColors.Reset();
			InterpolatedMotionVectors.Reset();

			// Make sure our capacity fits the requested vertex count
			InterpolatedPositions.Reserve(NumVertices);
			InterpolatedTangentX.Reserve(NumVertices);
			InterpolatedTangentZ.Reserve(NumVertices);
			InterpolatedUVs.Reserve(NumVertices);
			InterpolatedColors.Reserve(NumVertices);

			InterpolatedPositions.AddUninitialized(NumVertices);
			InterpolatedTangentX.AddUninitialized(NumVertices);
			InterpolatedTangentZ.AddUninitialized(NumVertices);
			InterpolatedUVs.AddUninitialized(NumVertices);
			InterpolatedColors.AddUninitialized(NumVertices);

			if (bHasMotionVectors)
			{
				InterpolatedMotionVectors.Reserve(NumVertices);
				InterpolatedMotionVectors.AddUninitialized(NumVertices);
			}
		}

		void Empty()
		{
			// Clear entries but and release memory.
			InterpolatedPositions.Empty();
			InterpolatedTangentX.Empty();
			InterpolatedTangentZ.Empty();
			InterpolatedUVs.Empty();
			InterpolatedColors.Empty();
			InterpolatedMotionVectors.Empty();
		}
	}
	mutable Scratch;

	uint32 UpdatedFrameNum;
	float Time;
	float PlaybackSpeed;
	float MotionVectorScale;

	bool bOverrideWireframeColor = false;
	FLinearColor WireframeOverrideColor = FLinearColor::Green;

	FMaterialRelevance MaterialRelevance;
	uint32 bLooping : 1;
	uint32 bIsPlayingBackwards : 1;
	uint32 bExtrapolateFrames : 1;
	uint32 bInterpolateFrames : 1;

	/** Function used to create a new track proxy at construction */
	TFunction<FGeomCacheTrackProxy*()> CreateTrackProxy;
#if RHI_RAYTRACING
	FName RayTracingDebugName = NAME_None;
#endif
};

#if !defined(GEOMETRY_CACHE_SCENE_PROXY_ISPC_ENABLED_DEFAULT)
#define GEOMETRY_CACHE_SCENE_PROXY_ISPC_ENABLED_DEFAULT 1
#endif

#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool GGeometryCacheSceneProxyUseIspc = INTEL_ISPC && GEOMETRY_CACHE_SCENE_PROXY_ISPC_ENABLED_DEFAULT;
#else
extern bool GGeometryCacheSceneProxyUseIspc;
#endif

#undef UE_API
