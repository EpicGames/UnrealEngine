// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryGeometryCache.generated.h"

class AActor;
struct FAssetData;

/** Factory class for spawning and creating GeometryCacheActors */
UCLASS(MinimalAPI, config = Editor)
class UActorFactoryGeometryCache : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	// End UActorFactory Interface
};
