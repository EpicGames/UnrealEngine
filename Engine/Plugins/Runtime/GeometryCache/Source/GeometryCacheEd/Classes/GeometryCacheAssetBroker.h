// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentAssetBroker.h"

/** AssetBroker class for GeometryCache assets*/
class FGeometryCacheAssetBroker : public IComponentAssetBroker
{
public:
	// Begin IComponentAssetBroker Implementation
	UClass* GetSupportedAssetClass() override;
	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override;
	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override;
	// End IComponentAssetBroker Implementation
};
