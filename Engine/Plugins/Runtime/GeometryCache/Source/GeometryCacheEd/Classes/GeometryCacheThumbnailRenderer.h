// Copyright Epic Games, Inc. All Rights Reserved.

/**
*
* This thumbnail renderer displays a given GeometryCache
*/

#pragma once

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "GeometryCacheThumbnailRenderer.generated.h"

enum class EThumbnailRenderFrequency : uint8;

class FCanvas;
class FGeometryCacheThumbnailScene;
class FRenderTarget;

UCLASS(config = Editor, MinimalAPI)
class UGeometryCacheThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// Begin UObject implementation
	virtual void BeginDestroy() override;
	// End UObject implementation
private:
	FGeometryCacheThumbnailScene* ThumbnailScene;
public:
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;

};
