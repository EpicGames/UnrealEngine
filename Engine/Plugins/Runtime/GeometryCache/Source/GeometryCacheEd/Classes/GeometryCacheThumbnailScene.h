// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailHelpers.h"

class AGeometryCacheActor;
class UGeometryCache;

class FGeometryCacheThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	FGeometryCacheThumbnailScene();

	/** Sets the static mesh to use in the next CreateView() */
	void SetGeometryCache(UGeometryCache* GeometryCache);

protected:
	// FThumbnailPreviewScene implementation
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The static mesh actor used to display all static mesh thumbnails */
	AGeometryCacheActor* PreviewActor;
};
