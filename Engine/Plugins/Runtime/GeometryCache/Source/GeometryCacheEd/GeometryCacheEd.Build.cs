// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryCacheEd : ModuleRules
{
	public GeometryCacheEd(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
                "InputCore",
                "RenderCore",
                "RHI",
				"EditorFramework",
				"EditorWidgets",
				"UnrealEd",
				"AssetTools",
                "GeometryCache",
				"ToolMenus",
				"NiagaraEditor",
				"AdvancedPreviewScene",
				"SequencerWidgets",
				"TimeManagement"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
