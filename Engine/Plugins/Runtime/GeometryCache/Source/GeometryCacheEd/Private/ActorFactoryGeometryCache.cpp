// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryGeometryCache.h"
#include "AssetRegistry/AssetData.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryGeometryCache)

UActorFactoryGeometryCache::UActorFactoryGeometryCache(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	DisplayName = FText::FromString( "Geometry Cache" );
	NewActorClass = AGeometryCacheActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryGeometryCache::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UGeometryCache::StaticClass()))
	{
		OutErrorMsg = FText::FromString("A valid GeometryCache must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryGeometryCache::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UGeometryCache* GeometryCache = CastChecked<UGeometryCache>(Asset);

	// Change properties
	AGeometryCacheActor* GeometryCacheActor = CastChecked<AGeometryCacheActor>(NewActor);
	UGeometryCacheComponent* GeometryCacheComponent = GeometryCacheActor->GetGeometryCacheComponent();
	check(GeometryCacheComponent);

	GeometryCacheComponent->UnregisterComponent();

	// Set GeometryCache (data) instance
	GeometryCacheComponent->GeometryCache = GeometryCache;

	// Init Component
	GeometryCacheComponent->RegisterComponent();
}

