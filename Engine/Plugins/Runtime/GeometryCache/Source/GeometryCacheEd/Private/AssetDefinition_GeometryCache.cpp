// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_GeometryCache.h"

#include "EditorFramework/AssetImportData.h"
#include "GeometryCache.h"
#include "GeometryCacheAssetEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_GeometryCache)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_GeometryCache::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_GeometryCache", "GeometryCache");
}

FLinearColor UAssetDefinition_GeometryCache::GetAssetColor() const
{
	return FColor(0, 255, 255);
}

TSoftClassPtr<UObject> UAssetDefinition_GeometryCache::GetAssetClass() const
{
	return UGeometryCache::StaticClass();
}

bool UAssetDefinition_GeometryCache::CanImport() const
{
	return true;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_GeometryCache::GetAssetCategories() const
{
	static const auto Categories = {EAssetCategoryPaths::Animation};
	return Categories;
}

EAssetCommandResult UAssetDefinition_GeometryCache::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UGeometryCache* GeometryCacheAsset : OpenArgs.LoadObjects<UGeometryCache>())
	{
		if (GeometryCacheAsset != nullptr)
		{
			TSharedRef<FGeometryCacheAssetEditorToolkit> NewCustomAssetEditor(new FGeometryCacheAssetEditorToolkit());
			NewCustomAssetEditor->InitCustomAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, GeometryCacheAsset);
		}
	}
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
