// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAssetBroker.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"

UClass* FGeometryCacheAssetBroker::GetSupportedAssetClass()
{
	return UGeometryCache::StaticClass();
}

bool FGeometryCacheAssetBroker::AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset)
{
	if (UGeometryCacheComponent* GeometryCacheComponent = Cast<UGeometryCacheComponent>(InComponent))
	{
		UGeometryCache* GeomCache = Cast<UGeometryCache>(InAsset);

		if ((GeomCache != nullptr) || (InAsset == nullptr))
		{
			GeometryCacheComponent->SetGeometryCache(GeomCache);

			return true;
		}
	}

	return false;
}

UObject* FGeometryCacheAssetBroker::GetAssetFromComponent(UActorComponent* InComponent)
{
	if (UGeometryCacheComponent* GeometryCacheComponent = Cast<UGeometryCacheComponent>(InComponent))
	{
		return GeometryCacheComponent->GetGeometryCache();
	}
	return nullptr;
}
