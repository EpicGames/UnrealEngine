// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAssetEditorToolkit.h"

#include "AdvancedPreviewSceneModule.h"
#include "CoreMinimal.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheTimelineBindingAsset.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SEditorViewport.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SGeometryCacheEditorViewport.h"
#include "Widgets/SGeometryCacheTimeline.h"

#define LOCTEXT_NAMESPACE "GeometryCacheCustomAssetEditor"

namespace UE::GeometryCacheAssetEditorToolkit::Private
{
	static const FName GeometryCacheEditorAppIdentifier(TEXT("GeometryCacheEditor"));
	static const FName ToolkitFName(TEXT("GeometryCacheEditor"));
	static const FName TabId_Viewport(TEXT("GeometryCacheCustomAssetEditor_Render"));
	static const FName TabId_AssetProperties(TEXT("GeometryCacheCustomAssetEditor_Details"));
	static const FName TabId_AnimationProperties(TEXT("GeometryCacheCustomAssetEditor_Timeline"));
	static const FName TabId_PreviewSceneProperties(TEXT("GeometryCacheCustomAssetEditor_PreviewScene"));
}

void FGeometryCacheAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	using namespace UE::GeometryCacheAssetEditorToolkit::Private;

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenuGeometryCacheEditor", "Geometry Cache Asset Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(TabId_Viewport, FOnSpawnTab::CreateSP(this, &FGeometryCacheAssetEditorToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Render"));

	InTabManager->RegisterTabSpawner(TabId_AssetProperties, FOnSpawnTab::CreateSP(this, &FGeometryCacheAssetEditorToolkit::SpawnTab_AssetProperties))
		.SetDisplayName(LOCTEXT("AssetPropertiesTab", "Properties"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(TabId_AnimationProperties, FOnSpawnTab::CreateSP(this, &FGeometryCacheAssetEditorToolkit::SpawnTab_AnimationProperties))
		.SetDisplayName(LOCTEXT("AnimationPropertiesTab", "Timeline"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CurveBase"));

	InTabManager->RegisterTabSpawner(TabId_PreviewSceneProperties, FOnSpawnTab::CreateSP(this, &FGeometryCacheAssetEditorToolkit::SpawnTab_PreviewSceneProperties))
		.SetDisplayName(LOCTEXT("PreviewScenePropertiesTab", "Preview Settings"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FGeometryCacheAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	using namespace UE::GeometryCacheAssetEditorToolkit::Private;

	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(TabId_Viewport);
	InTabManager->UnregisterTabSpawner(TabId_AssetProperties);
	InTabManager->UnregisterTabSpawner(TabId_AnimationProperties);
	InTabManager->UnregisterTabSpawner(TabId_PreviewSceneProperties);
}

void FGeometryCacheAssetEditorToolkit::InitCustomAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UGeometryCache* InCustomAsset)
{
	using namespace UE::GeometryCacheAssetEditorToolkit::Private;

	GeometryCacheAsset = InCustomAsset;

	ViewportTab = SNew(SGeometryCacheEditorViewport);
	
	InitPreviewComponents();
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	DetailView_AssetProperties = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_GeometryCacheEditor_Layout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.9f)
						->SetHideTabWell(true)
						->AddTab(TabId_Viewport, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->SetHideTabWell(true)
						->AddTab(TabId_AnimationProperties, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(TabId_AssetProperties, ETabState::OpenedTab)
					->AddTab(TabId_PreviewSceneProperties, ETabState::OpenedTab)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;

	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		GeometryCacheEditorAppIdentifier,
		StandaloneDefaultLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		(UObject*)InCustomAsset);

	if (DetailView_AssetProperties.IsValid())
	{
		DetailView_AssetProperties->SetObject(Cast<UObject>(GeometryCacheAsset));
	}
}

FGeometryCacheAssetEditorToolkit::FGeometryCacheAssetEditorToolkit()
	: GeometryCacheAsset(nullptr)
{
}

FName FGeometryCacheAssetEditorToolkit::GetToolkitFName() const
{
	return UE::GeometryCacheAssetEditorToolkit::Private::ToolkitFName;
}

FText FGeometryCacheAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Geometry Cache Asset Editor");
}

FText FGeometryCacheAssetEditorToolkit::GetToolkitName() const
{
	return FText::FromString(GeometryCacheAsset->GetName());
}

FText FGeometryCacheAssetEditorToolkit::GetToolkitToolTipText() const
{
	return LOCTEXT("ToolTip", "Geometry Cache Asset Editor");
}

FString FGeometryCacheAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "GeometryCache ").ToString();
}

FLinearColor FGeometryCacheAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FColor(0, 255, 255);
}

void FGeometryCacheAssetEditorToolkit::OnClose()
{
	ViewportTab.Reset();
	DetailView_AssetProperties.Reset();
}

void FGeometryCacheAssetEditorToolkit::InitPreviewComponents()
{
	if (GeometryCacheAsset == nullptr)
	{
		return;
	}

	if (PreviewGeometryCacheComponent == nullptr)
	{
		PreviewGeometryCacheComponent = NewObject<UGeometryCacheComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		PreviewGeometryCacheComponent->CastShadow = 1;
		PreviewGeometryCacheComponent->bCastDynamicShadow = 1;
		PreviewGeometryCacheComponent->SetGeometryCache(GeometryCacheAsset.Get());
		PreviewGeometryCacheComponent->Activate(true);
	}
	
	BindingAsset = MakeShareable(new FGeometryCacheTimelineBindingAsset(PreviewGeometryCacheComponent));
}

TSharedRef<SDockTab> FGeometryCacheAssetEditorToolkit::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == UE::GeometryCacheAssetEditorToolkit::Private::TabId_Viewport);
	check(PreviewGeometryCacheComponent != nullptr);

	ViewportTab->SetGeometryCacheComponent(PreviewGeometryCacheComponent.Get());

	return SNew(SDockTab)
		.Label(LOCTEXT("ViewportTab", "Viewport"))
		.TabColorScale(GetTabColorScale())
		[
			ViewportTab.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGeometryCacheAssetEditorToolkit::SpawnTab_AssetProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == UE::GeometryCacheAssetEditorToolkit::Private::TabId_AssetProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("AssetPropertiesTab", "Properties"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_AssetProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGeometryCacheAssetEditorToolkit::SpawnTab_AnimationProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == UE::GeometryCacheAssetEditorToolkit::Private::TabId_AnimationProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("AnimationPropertiesTab", "Timeline"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SGeometryCacheTimeline, BindingAsset.ToSharedRef())
		];
}

TSharedRef<SDockTab> FGeometryCacheAssetEditorToolkit::SpawnTab_PreviewSceneProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == UE::GeometryCacheAssetEditorToolkit::Private::TabId_PreviewSceneProperties);

	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>(TEXT("AdvancedPreviewScene"));

	TSharedRef<SWidget> PreviewSceneSettingsWidget = SNullWidget::NullWidget;
	TSharedPtr<class FAdvancedPreviewScene> PreviewScene = ViewportTab->GetAdvancedPreviewScene();
	if (PreviewScene.IsValid())
	{
		PreviewSceneSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());
	}

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		[
			SNew(SBox)
				[
					PreviewSceneSettingsWidget
				]
		];

	return SpawnedTab;
}

#undef LOCTEXT_NAMESPACE