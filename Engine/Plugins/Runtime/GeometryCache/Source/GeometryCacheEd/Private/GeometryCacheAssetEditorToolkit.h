// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "GeometryCache.h"

class IDetailsView;
class SDockableTab;
class UGeometryCacheComponent;
class SGeometryCacheTimeline;
class SGeometryCacheEditorViewport;
class FGeometryCacheTimelineBindingAsset;

class FGeometryCacheAssetEditorToolkit : public FAssetEditorToolkit
{
public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;

	/**
	 * Edits the specified asset object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InCustomAsset			The Custom Asset to Edit
	 */
	void InitCustomAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UGeometryCache * InCustomAsset);

	FGeometryCacheAssetEditorToolkit();
	virtual ~FGeometryCacheAssetEditorToolkit() = default;

	/** Begin IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual bool IsPrimaryEditor() const override { return true; }
	virtual void OnClose() override;
	/** End IToolkit interface */

private:
	void InitPreviewComponents();

private:
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AssetProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AnimationProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PreviewSceneProperties(const FSpawnTabArgs& Args);

private:
	TWeakObjectPtr<UGeometryCache> GeometryCacheAsset;
	TWeakObjectPtr<UGeometryCacheComponent> PreviewGeometryCacheComponent;

	TSharedPtr<SGeometryCacheEditorViewport> ViewportTab;
	TSharedPtr<IDetailsView> DetailView_AssetProperties;
	TSharedPtr<FGeometryCacheTimelineBindingAsset> BindingAsset;
};