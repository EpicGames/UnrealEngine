// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheEdModule.h"

#include "GeometryCache.h"
#include "GeometryCacheAssetBroker.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheModule.h"
#include "GeometryCacheThumbnailRenderer.h"
#include "NiagaraEditorModule.h"
#include "NiagaraGeometryCacheRendererProperties.h"
#include "ThumbnailRendering/ThumbnailManager.h"

IMPLEMENT_MODULE(FGeometryCacheEdModule, GeometryCacheEd)

void FGeometryCacheEdModule::StartupModule()
{
	LLM_SCOPE_BYTAG(GeometryCache);

	AssetBroker = new FGeometryCacheAssetBroker();
	FComponentAssetBrokerage::RegisterBroker(MakeShareable(AssetBroker), UGeometryCacheComponent::StaticClass(), true, true);

	UThumbnailManager::Get().RegisterCustomRenderer(UGeometryCache::StaticClass(), UGeometryCacheThumbnailRenderer::StaticClass());

	FNiagaraEditorModule& NiagaraEditorModule = FNiagaraEditorModule::Get();
	NiagaraEditorModule.RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraGeometryCacheRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraGeometryCacheRendererProperties::StaticClass()->GetDescription()),
		UNiagaraGeometryCacheRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraGeometryCacheRendererProperties* NewRenderer = NewObject<UNiagaraGeometryCacheRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			if(ensure(NewRenderer->GeometryCaches.Num() == 1))
			{
				FSoftObjectPath DefaultGeometryCache(TEXT("GeometryCache'/Niagara/DefaultAssets/GeometryCache/DefaultGeometryCacheAsset.DefaultGeometryCacheAsset'"));
				NewRenderer->GeometryCaches[0].GeometryCache = Cast<UGeometryCache>(DefaultGeometryCache.TryLoad());
			}
			return NewRenderer;
		})));
}

void FGeometryCacheEdModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		FComponentAssetBrokerage::UnregisterBroker(MakeShareable(AssetBroker));
		UThumbnailManager::Get().UnregisterCustomRenderer(UGeometryCache::StaticClass());
	}
}
