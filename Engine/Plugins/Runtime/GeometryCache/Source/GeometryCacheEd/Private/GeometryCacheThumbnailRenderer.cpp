// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheThumbnailRenderer.h"
#include "SceneView.h"
#include "GeometryCacheThumbnailScene.h"
#include "GeometryCache.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheThumbnailRenderer)

UGeometryCacheThumbnailRenderer::UGeometryCacheThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

void UGeometryCacheThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UGeometryCache* GeometryCache = Cast<UGeometryCache>(Object);
	if (IsValid(GeometryCache))
	{
		if (ThumbnailScene == nullptr)
		{
			ThumbnailScene = new FGeometryCacheThumbnailScene();
		}

		ThumbnailScene->SetGeometryCache(GeometryCache);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetGeometryCache(nullptr);
	}
}

void UGeometryCacheThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}

EThumbnailRenderFrequency UGeometryCacheThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	return EThumbnailRenderFrequency::OnPropertyChange;
}

