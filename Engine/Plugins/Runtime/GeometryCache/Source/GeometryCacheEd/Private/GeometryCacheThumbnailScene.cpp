// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheThumbnailScene.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheActor.h"

FGeometryCacheThumbnailScene::FGeometryCacheThumbnailScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AGeometryCacheActor>(SpawnInfo);

	PreviewActor->GetGeometryCacheComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FGeometryCacheThumbnailScene::SetGeometryCache(UGeometryCache* GeometryCache)
{
	PreviewActor->GetGeometryCacheComponent()->SetGeometryCache(GeometryCache);

	if (GeometryCache)
	{
		FTransform MeshTransform = FTransform::Identity;

		PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
		PreviewActor->GetGeometryCacheComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetGeometryCacheComponent()->Bounds);
		PreviewActor->SetActorLocation(-PreviewActor->GetGeometryCacheComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
		PreviewActor->GetGeometryCacheComponent()->RecreateRenderState_Concurrent();
	}
}

void FGeometryCacheThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetGeometryCacheComponent());
	check(PreviewActor->GetGeometryCacheComponent()->GeometryCache);

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetGeometryCacheComponent()->Bounds.SphereRadius * 1.15);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetGeometryCacheComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetGeometryCacheComponent()->GeometryCache->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

