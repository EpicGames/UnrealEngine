// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTimelineBindingAsset.h"

#include "CommonFrameRates.h"
#include "Fonts/FontMeasure.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheHelpers.h"
#include "Styling/AppStyle.h"

FGeometryCacheTimelineBindingAsset::FGeometryCacheTimelineBindingAsset(TWeakObjectPtr<UGeometryCacheComponent> InPreviewComponent)
	:PreviewComponent(InPreviewComponent)
{
	if (InPreviewComponent.IsValid())
	{
		PlaybackRange = FAnimatedRange(0.0, (double)InPreviewComponent->GetDuration());
		SetViewRange(PlaybackRange);
	}
}

FFrameRate FGeometryCacheTimelineBindingAsset::GetFrameRate() const
{
	return FFrameRate(FMath::RoundToInt32((float)PreviewComponent.Pin()->GetNumberOfFrames() / PreviewComponent.Pin()->GetDuration()), 1);
}

int32 FGeometryCacheTimelineBindingAsset::GetTickResolution() const
{
	return int32(1000);
}

FAnimatedRange FGeometryCacheTimelineBindingAsset::GetViewRange() const
{
	return ViewRange;
}

void FGeometryCacheTimelineBindingAsset::SetViewRange(TRange<double> InRange)
{
	ViewRange = InRange;

	if (WorkingRange.HasLowerBound() && WorkingRange.HasUpperBound())
	{
		WorkingRange = TRange<double>::Hull(WorkingRange, ViewRange);
	}
	else
	{
		WorkingRange = ViewRange;
	}
}

FAnimatedRange FGeometryCacheTimelineBindingAsset::GetWorkingRange() const
{
	return WorkingRange;
}

TRange<FFrameNumber> FGeometryCacheTimelineBindingAsset::GetPlaybackRange() const
{
	const int32 Resolution = GetTickResolution();
	return TRange<FFrameNumber>(FFrameNumber(FMath::RoundToInt32(PlaybackRange.GetLowerBoundValue() * (double)Resolution)), FFrameNumber(FMath::RoundToInt32(PlaybackRange.GetUpperBoundValue() * (double)Resolution)));
}

FFrameNumber FGeometryCacheTimelineBindingAsset::GetScrubPosition() const
{
	if (PreviewComponent.IsValid())
	{
		float SampleTime = 0.0f;
		if(PreviewComponent->IsPlaying() && PreviewComponent->IsLooping())
		{
			SampleTime = GeometyCacheHelpers::WrapAnimationTime(PreviewComponent->GetElapsedTime(), PreviewComponent->GetDuration());
		}
		else
		{
			SampleTime = PreviewComponent->GetElapsedTime();
			if (PreviewComponent->GetDuration() > UE_SMALL_NUMBER)
			{
				while (SampleTime > PreviewComponent->GetDuration())
				{
					SampleTime -= PreviewComponent->GetDuration();
				}
				while (SampleTime < 0)
				{
					SampleTime += PreviewComponent->GetDuration();
				}
			}
			SampleTime = FMath::Clamp(SampleTime, 0.0f, PreviewComponent->GetDuration());
		}

		return FFrameNumber(FMath::RoundToInt32(SampleTime * (double)GetTickResolution()));
	}

	return FFrameNumber(0);
}

float FGeometryCacheTimelineBindingAsset::GetScrubTime() const
{
	if (PreviewComponent.IsValid())
	{
		return GeometyCacheHelpers::WrapAnimationTime(PreviewComponent->GetElapsedTime(), PreviewComponent->GetDuration());
	}

	return 0.0f;
}

void FGeometryCacheTimelineBindingAsset::SetScrubPosition(FFrameTime NewScrubPosition) const
{
	if (PreviewComponent.IsValid())
	{
		if (PreviewComponent->IsPlaying())
		{
			PreviewComponent->Stop();
		}

		PreviewComponent->SetCurrentTime(static_cast<float>(NewScrubPosition.AsDecimal() / static_cast<double>(GetTickResolution())));
	}
}

void FGeometryCacheTimelineBindingAsset::HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation)
{
	SetViewRange(InRange);
}

void FGeometryCacheTimelineBindingAsset::HandleWorkingRangeChanged(TRange<double> InRange)
{
	WorkingRange = InRange;
}