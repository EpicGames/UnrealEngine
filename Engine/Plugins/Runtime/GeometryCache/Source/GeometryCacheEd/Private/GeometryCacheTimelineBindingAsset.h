// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ITimeSlider.h"
#include "TimeSliderArgs.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"

class UGeometryCacheComponent;

class FGeometryCacheTimelineBindingAsset : public TSharedFromThis<FGeometryCacheTimelineBindingAsset>
{
public:
	FGeometryCacheTimelineBindingAsset(TWeakObjectPtr<UGeometryCacheComponent> InPreviewComponent);
	
	TWeakObjectPtr<UGeometryCacheComponent> GetPreviewComponent() const { return PreviewComponent; }

	/** Get the framerate specified by the anim sequence */
	FFrameRate GetFrameRate() const;

	/** Get the tick resolution we are displaying at */
	int32 GetTickResolution() const;

	/** Get the current view range */
	FAnimatedRange GetViewRange() const;

	/** Set the current view range */
	void SetViewRange(TRange<double> InRange);

	/** Get the working range of the model's data */
	FAnimatedRange GetWorkingRange() const;

	/** Get the playback range of the model's data */
	TRange<FFrameNumber> GetPlaybackRange() const;

	/** Get the current scrub position */
	FFrameNumber GetScrubPosition() const;

	/** Get the current scrub time */
	float GetScrubTime() const;

	/** Set the current scrub position */
	void SetScrubPosition(FFrameTime NewScrubPostion) const;

	/** Handle the view range being changed */
	void HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation);

	/** Handle the working range being changed */
	void HandleWorkingRangeChanged(TRange<double> InRange);

private:
	TWeakObjectPtr<UGeometryCacheComponent> PreviewComponent;

	FAnimatedRange ViewRange;

	FAnimatedRange WorkingRange;

	FAnimatedRange PlaybackRange;

};