// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCacheEditorViewport.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "AssetViewerSettings.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "FinalPostProcessSettings.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheMeshData.h"
#include "SceneView.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "Slate/SceneViewport.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SGeometryCacheEditorViewport"

////// VIEWPORT CLIENT	////////////////

class FGeometryCacheEditorViewportClient : public FEditorViewportClient
{
public:
	FGeometryCacheEditorViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SGeometryCacheEditorViewport>& InGeometryCacheEditorViewport);
	virtual ~FGeometryCacheEditorViewportClient() override;

	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override { return true; }

	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;

	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }

private:
	TWeakPtr<SGeometryCacheEditorViewport> GeometryCacheEditorViewportPtr;
};

FGeometryCacheEditorViewportClient::FGeometryCacheEditorViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SGeometryCacheEditorViewport>& InGeometryCacheEditorViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InGeometryCacheEditorViewport))
{
	GeometryCacheEditorViewportPtr = InGeometryCacheEditorViewport;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = false;
	DrawHelper.GridColorAxis = FColor(80, 80, 80);
	DrawHelper.GridColorMajor = FColor(72, 72, 72);
	DrawHelper.GridColorMinor = FColor(64, 64, 64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	ShowWidget(false);

	SetViewMode(VMI_Lit);

	//EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetTemporalAA(true);
	EngineShowFlags.SetShaderPrint(true);
	
	OverrideNearClipPlane(0.001f);

	//This seems to be needed to get the correct world time in the preview.
	SetIsSimulateInEditorViewport(true);

	UEditorPerProjectUserSettings* PerProjectSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	UAssetViewerSettings* DefaultSettings = UAssetViewerSettings::Get();
	PerProjectSettings->AssetViewerProfileIndex = DefaultSettings->Profiles.IsValidIndex(PerProjectSettings->AssetViewerProfileIndex) ? PerProjectSettings->AssetViewerProfileIndex : 0;

	FAdvancedPreviewScene* PreviewSceneCasted = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	PreviewSceneCasted->SetProfileIndex(PerProjectSettings->AssetViewerProfileIndex);
}

FGeometryCacheEditorViewportClient::~FGeometryCacheEditorViewportClient()
{
}

FLinearColor FGeometryCacheEditorViewportClient::GetBackgroundColor() const
{
	FLinearColor BackgroundColor = FLinearColor::Black;
	return BackgroundColor;
}

FSceneView* FGeometryCacheEditorViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	FSceneView* SceneView = FEditorViewportClient::CalcSceneView(ViewFamily);
	FFinalPostProcessSettings::FCubemapEntry& CubemapEntry = *new(SceneView->FinalPostProcessSettings.ContributingCubemaps) FFinalPostProcessSettings::FCubemapEntry;
	CubemapEntry.AmbientCubemap = GUnrealEd->GetThumbnailManager()->AmbientCubemap;
	CubemapEntry.AmbientCubemapTintMulScaleValue = FLinearColor::White;
	return SceneView;
}

void FGeometryCacheEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

void FGeometryCacheEditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	FEditorViewportClient::Draw(InViewport, Canvas);
}

////// EDITOR			////////////////

void SGeometryCacheEditorViewport::Construct(const FArguments& InArgs)
{
	PreviewGeometryCacheComponent = nullptr;

	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));

	SEditorViewport::Construct(SEditorViewport::FArguments());
	
	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(AdvancedPreviewScene, Client);
	
	AdvancedPreviewScene->SetFloorVisibility(false, /* bDirect */ true);
}

void SGeometryCacheEditorViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PreviewGeometryCacheComponent != nullptr)
	{
		Collector.AddReferencedObject(PreviewGeometryCacheComponent);
	}
}

void SGeometryCacheEditorViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SEditorViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedRef<class SEditorViewport> SGeometryCacheEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SGeometryCacheEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SGeometryCacheEditorViewport::OnFloatingButtonClicked()
{
}

void SGeometryCacheEditorViewport::SetGeometryCacheComponent(UGeometryCacheComponent* InGeometryCacheComponent)
{
	if (PreviewGeometryCacheComponent != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(PreviewGeometryCacheComponent);
	}

	PreviewGeometryCacheComponent = InGeometryCacheComponent;

	if (PreviewGeometryCacheComponent != nullptr)
	{
		AdvancedPreviewScene->AddComponent(PreviewGeometryCacheComponent, PreviewGeometryCacheComponent->GetRelativeTransform());
	}

	if (PreviewGeometryCacheComponent != nullptr && SystemViewportClient)
	{
		SystemViewportClient->FocusViewportOnBox(PreviewGeometryCacheComponent->Bounds.GetBox());
	}

	SceneViewport->Invalidate();

}

TSharedRef<FEditorViewportClient> SGeometryCacheEditorViewport::MakeEditorViewportClient()
{
	SystemViewportClient = MakeShareable(new FGeometryCacheEditorViewportClient(*AdvancedPreviewScene.Get(), SharedThis(this)));

	SystemViewportClient->SetViewLocation(FVector::ZeroVector);
	SystemViewportClient->SetViewRotation(FRotator::ZeroRotator);
	SystemViewportClient->SetViewLocationForOrbiting(FVector::ZeroVector);
	SystemViewportClient->bSetListenerPosition = false;

	SystemViewportClient->SetRealtime(true);
	SystemViewportClient->VisibilityDelegate.BindSP(this, &SGeometryCacheEditorViewport::IsVisible);

	return SystemViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SGeometryCacheEditorViewport::BuildViewportToolbar()
{
	const FName ViewportToolbarName = "GeometryCacheEditor.ViewportToolbar";

	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowLensControls()));

			// Add the "View Modes" sub menu.
			RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			
			// Add the "Show" submenu.
			RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
			
			// Performance and Scalability Submenu
			{
				// Add Performance and Scalability Menu
				RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());

				const FName SubmenuName = UToolMenus::JoinMenuPaths(ViewportToolbarName, "PerformanceAndScalability");
				UToolMenu* Submenu = UToolMenus::Get()->ExtendMenu(SubmenuName);
				FToolMenuSection& Section = Submenu->FindOrAddSection("PerformanceAndScalability");

				// Add Scalability Menu
				Section.AddEntry(UE::UnrealEd::CreateScalabilitySubmenu());
			}
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
			ContextObject->Viewport = SharedThis(this);
			ViewportToolbarContext.AddObject(ContextObject);
		}
	}

	
	const TSharedRef<SWidget> NewViewportToolbar = SNew(SBox)
		[
			UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext)
		];
	
	return NewViewportToolbar;
}

EVisibility SGeometryCacheEditorViewport::OnGetViewportContentVisibility() const
{
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGeometryCacheEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();
}

void SGeometryCacheEditorViewport::OnFocusViewportToSelection()
{
	if (PreviewGeometryCacheComponent)
	{
		SystemViewportClient->FocusViewportOnBox(PreviewGeometryCacheComponent->Bounds.GetBox());
	}
}

void SGeometryCacheEditorViewport::PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)
		.Padding(FMargin(4.0f, 3.0f, 0.0f, 0.0f))
		[
			SNew(SRichTextBlock)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.DecoratorStyleSet(&FAppStyle::Get())
				.Text(this, &SGeometryCacheEditorViewport::BuildStatsText)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimViewport.MessageText"))
		];
}

bool SGeometryCacheEditorViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground()) && SEditorViewport::IsVisible();
}

FText SGeometryCacheEditorViewport::BuildStatsText() const
{
	if (!PreviewGeometryCacheComponent)
	{
		return FText::FromString("No component to preview");
	}
		
	const FText CompName = FText::FromString(PreviewGeometryCacheComponent->GetName());
	const int32 Tracks = PreviewGeometryCacheComponent->GetNumberOfTracks();
	const int32 Frames = PreviewGeometryCacheComponent->GetNumberOfFrames();
	const float Duration = PreviewGeometryCacheComponent->GetDuration();
	const FText DurationText = FText::FromString(FString::Printf(TEXT("%.2f s"), Duration));
	const int32 FPS = FMath::FloorToInt((float)Frames / Duration);
	int32 NumTriangles = 0;

	if (UGeometryCache* GeometryCache = PreviewGeometryCacheComponent->GetGeometryCache())
	{
		constexpr float Time = 0.0f;
		TArray<FGeometryCacheMeshData> MeshData;
		GeometryCache->GetMeshDataAtTime(Time, MeshData);
		if (MeshData.Num() > 0)
		{
			NumTriangles = MeshData[0].Positions.Num() / 3;
		}
	}

	return FText::FormatNamed(LOCTEXT("StatsText", "Previewing {CompName}\nTracks: {Tracks}\nFrames: {Frames}\nDuration:{Duration}\nFPS: {FPS} fps\nTriangles: {NumTriangles}"),
		TEXT("CompName"), CompName,
		TEXT("Tracks"), Tracks,
		TEXT("Frames"), Frames,
		TEXT("Duration"), DurationText,
		TEXT("FPS"), FPS,
		TEXT("NumTriangles"), NumTriangles);
}

#undef LOCTEXT_NAMESPACE