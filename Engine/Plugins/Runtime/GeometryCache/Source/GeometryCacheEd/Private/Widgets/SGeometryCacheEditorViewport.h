// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/GCObject.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PreviewScene.h"
#include "Framework/Commands/UICommandList.h"
#include "EditorViewportClient.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "HairStrandsInterface.h"

class FGeometryCacheEditorViewportClient;
class FAdvancedPreviewScene;
class UGeometryCacheComponent;

class SGeometryCacheEditorViewport : public SEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider
{
public:
	void Construct(const FArguments& InArgs);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SGeometryCacheEditorViewport");
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	// Set the component to preview
	void SetGeometryCacheComponent(UGeometryCacheComponent* InGeometryCacheComponent);

	TSharedPtr<FAdvancedPreviewScene> GetAdvancedPreviewScene() { return AdvancedPreviewScene; }
	
protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	virtual void PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay) override;

private:
	bool IsVisible() const override;

	FText BuildStatsText() const;
private:
	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

	/** Level viewport client */
	TSharedPtr<FGeometryCacheEditorViewportClient> SystemViewportClient;

	/** Preview Scene - uses advanced preview settings */
	TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene;

	TObjectPtr<UGeometryCacheComponent> PreviewGeometryCacheComponent;
};