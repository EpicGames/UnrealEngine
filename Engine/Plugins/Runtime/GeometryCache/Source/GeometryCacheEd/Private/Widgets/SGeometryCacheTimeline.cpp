// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCacheTimeline.h"

#include "Fonts/FontMeasure.h"
#include "FrameNumberNumericInterface.h"
#include "FrameNumberTimeEvaluator.h"
#include "Framework/Application/SlateApplication.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheTimelineBindingAsset.h"
#include "GeometryCacheTimeSliderController.h"
#include "ISequencerWidgetsModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SGeometryCacheTimelineOverlay.h"
#include "Widgets/SGeometryCacheTimelineSplitterOverlay.h"
#include "Widgets/SGeometryCacheTimelineTransportControls.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SGeometryCacheTimeline"

void SGeometryCacheTimeline::Construct(const FArguments& InArgs, const TSharedRef<FGeometryCacheTimelineBindingAsset>& InBindingAsset)
{
	TWeakPtr<FGeometryCacheTimelineBindingAsset>  WeakBindingAsset = InBindingAsset;
	BindingAsset = InBindingAsset;

	const EFrameNumberDisplayFormats DisplayFormat = EFrameNumberDisplayFormats::Frames;
	const TAttribute<FFrameRate> TickResolution = MakeAttributeLambda([WeakBindingAsset]()
		{
			return FFrameRate(WeakBindingAsset.Pin()->GetTickResolution(), 1);
		});

	ViewRange = MakeAttributeLambda([WeakBindingAsset]() { return WeakBindingAsset.IsValid() ? WeakBindingAsset.Pin()->GetViewRange() : FAnimatedRange(0.0, 0.0); });
	
	const TAttribute<FFrameRate> DisplayRate = MakeAttributeLambda([WeakBindingAsset]()
		{
			return WeakBindingAsset.Pin()->GetFrameRate();
		});

	ColumnFillCoefficients[0] = 0.3f;
	ColumnFillCoefficients[1] = 0.7f;

	TAttribute<float> FillCoefficient_0, FillCoefficient_1;
	{
		FillCoefficient_0.Bind(TAttribute<float>::FGetter::CreateSP(this, &SGeometryCacheTimeline::GetColumnFillCoefficient, 0));
		FillCoefficient_1.Bind(TAttribute<float>::FGetter::CreateSP(this, &SGeometryCacheTimeline::GetColumnFillCoefficient, 1));
	};

	NumericTypeInterface = MakeShareable(new FFrameNumberInterface(DisplayFormat, 0, TickResolution, DisplayRate));

	FTimeSliderArgs TimeSliderArgs;
	{
		TimeSliderArgs.ScrubPosition = MakeAttributeLambda([WeakBindingAsset]() { return WeakBindingAsset.IsValid() ? WeakBindingAsset.Pin()->GetScrubPosition() : FFrameTime(0); });
		TimeSliderArgs.ViewRange = ViewRange;
		TimeSliderArgs.PlaybackRange = MakeAttributeLambda([WeakBindingAsset]() { return WeakBindingAsset.IsValid() ? WeakBindingAsset.Pin()->GetPlaybackRange() : TRange<FFrameNumber>(0, 0); });
		TimeSliderArgs.ClampRange = MakeAttributeLambda([WeakBindingAsset]() { return WeakBindingAsset.IsValid() ? WeakBindingAsset.Pin()->GetWorkingRange() : FAnimatedRange(0.0, 0.0); });
		TimeSliderArgs.DisplayRate = DisplayRate;
		TimeSliderArgs.TickResolution = TickResolution;
		TimeSliderArgs.OnViewRangeChanged = FOnViewRangeChanged::CreateSP(&InBindingAsset.Get(), &FGeometryCacheTimelineBindingAsset::HandleViewRangeChanged);
		TimeSliderArgs.OnClampRangeChanged = FOnTimeRangeChanged::CreateSP(&InBindingAsset.Get(), &FGeometryCacheTimelineBindingAsset::HandleWorkingRangeChanged);
		TimeSliderArgs.IsPlaybackRangeLocked = true;
		TimeSliderArgs.PlaybackStatus = EMovieScenePlayerStatus::Stopped;
		TimeSliderArgs.NumericTypeInterface = NumericTypeInterface;
		TimeSliderArgs.OnScrubPositionChanged = FOnScrubPositionChanged::CreateSP(this, &SGeometryCacheTimeline::HandleScrubPositionChanged);
	}

	TimeSliderController = MakeShareable(new FGeometryCacheTimeSlideController(TimeSliderArgs, SharedThis(this)));

	TSharedRef<FGeometryCacheTimeSlideController> TimeSliderControllerRef = TimeSliderController.ToSharedRef();

	// Create the top slider
	constexpr bool bMirrorLabels = false;
	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>("SequencerWidgets");
	TimeSlider = SequencerWidgets.CreateTimeSlider(TimeSliderControllerRef, bMirrorLabels);

	// Create bottom time range slider
	TSharedRef<ITimeSlider> BottomTimeRange = SequencerWidgets.CreateTimeRange(
		FTimeRangeArgs(
			EShowRange::ViewRange | EShowRange::WorkingRange | EShowRange::PlaybackRange,
			EShowRange::ViewRange | EShowRange::WorkingRange,
			TimeSliderControllerRef,
			EVisibility::Visible,
			NumericTypeInterface.ToSharedRef()
		),
		SequencerWidgets.CreateTimeRangeSlider(TimeSliderControllerRef)
	);

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar)
		.Thickness(FVector2D(5.0f, 5.0f));

	InitTrackNames();
	
	TracksListView = SNew(SListView< TSharedPtr<FString> >)
		.ExternalScrollbar(ScrollBar)
		.ListItemsSource(&TrackNames)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SGeometryCacheTimeline::HandleTimelineListViewGenerateRow);

	// Grid Panel Constants
	const int32 Column0 = 0, Column1 = 1;
	const int32 Row0 = 0, Row1 = 1, Row2 = 2, Row3 = 3, Row4 = 4;

	const float CommonPadding = 3.f;
	const FMargin ResizeBarPadding(4.f, 0, 0, 0);

	ChildSlot
		[
			SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SGridPanel)
						.FillRow(1, 1.0f)
						.FillColumn(0, FillCoefficient_0)
						.FillColumn(1, FillCoefficient_1)
						+ SGridPanel::Slot(Column1, Row0, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.BorderBackgroundColor(FLinearColor(.50f, .50f, .50f, 1.0f))
								.Padding(0.f)
								.Clipping(EWidgetClipping::ClipToBounds)
								[
									TimeSlider.ToSharedRef()
								]
						]
						+ SGridPanel::Slot(Column1, Row1, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SGeometryCacheTimelineOverlay, TimeSliderControllerRef)
								.Visibility(EVisibility::HitTestInvisible)
								.DisplayScrubPosition(false)
								.DisplayTickLines(true)
								.Clipping(EWidgetClipping::ClipToBounds)
								.PaintPlaybackRangeArgs(FPaintPlaybackRangeArgs(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_L"), FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_R"), 6.f))
						]

						// Overlay that draws the scrub position
						+ SGridPanel::Slot(Column1, Row1, SGridPanel::Layer(20))
						.Padding(ResizeBarPadding)
						[
							SNew(SGeometryCacheTimelineOverlay, TimeSliderControllerRef)
								.Visibility(EVisibility::HitTestInvisible)
								.DisplayScrubPosition(true)
								.DisplayTickLines(false)
								.Clipping(EWidgetClipping::ClipToBounds)
						]
						+ SGridPanel::Slot(Column1, Row3, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
								.Clipping(EWidgetClipping::ClipToBounds)
								.Padding(0)
								[
									BottomTimeRange
								]
						]
						+ SGridPanel::Slot(Column0, Row3, SGridPanel::Layer(10))
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SGeometryCacheTimelineTransportControls, InBindingAsset)
						]
						+ SGridPanel::Slot(Column0, Row1, SGridPanel::Layer(5))
						.ColumnSpan(2)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								[
									SNew(SOverlay)
										+ SOverlay::Slot()
										[
											SNew(SVerticalBox)
												+ SVerticalBox::Slot()
												.FillHeight(1.f)
												[
													SNew(SScrollBorder, TracksListView.ToSharedRef())
													[
														SNew(SHorizontalBox)
														+SHorizontalBox::Slot()
														.FillWidth(FillCoefficient_0)
														[
															SNew(SBox)
																[
																	TracksListView.ToSharedRef()
																]
														]
														+ SHorizontalBox::Slot()
														.FillWidth(FillCoefficient_1)
														[
															SNew(SBox)
															.Padding(ResizeBarPadding)
															.Clipping(EWidgetClipping::ClipToBounds)	
														]
													]
												]
										]
										+ SOverlay::Slot()
										.HAlign(HAlign_Right)
										[
											ScrollBar
										]
								]
						]
				]
				+ SOverlay::Slot()
				[
					// track area virtual splitter overlay
					SNew(SGeometryCacheTimelineSplitterOverlay)
						.Style(FAppStyle::Get(), "AnimTimeline.Outliner.Splitter")
						.Visibility(EVisibility::SelfHitTestInvisible)

						+ SSplitter::Slot()
						.Value(FillCoefficient_0)
						.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SGeometryCacheTimeline::OnColumnFillCoefficientChanged, 0))
						[
							SNew(SSpacer)
						]

						+ SSplitter::Slot()
						.Value(FillCoefficient_1)
						.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SGeometryCacheTimeline::OnColumnFillCoefficientChanged, 1))
						[
							SNew(SSpacer)
						]
				]
		];

}

void SGeometryCacheTimeline::HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate) const
{
	if (BindingAsset.IsValid())
	{
		TWeakObjectPtr<UGeometryCacheComponent> GeometryCacheComponent = BindingAsset.Pin()->GetPreviewComponent();
		if (GeometryCacheComponent.IsValid() && GeometryCacheComponent->IsPlaying())
		{
			GeometryCacheComponent->Stop();
		}

		BindingAsset.Pin()->SetScrubPosition(NewScrubPosition);
	}

}

void SGeometryCacheTimeline::InitTrackNames()
{
	TrackNames.Empty();
	if (BindingAsset.IsValid())
	{
		const TArray<FString> Names = BindingAsset.Pin()->GetPreviewComponent()->GetTrackNames();
		for (const FString& Name : Names)
		{
			TrackNames.Add(MakeShareable(new FString(Name)));
		}
	}

	
}

void SGeometryCacheTimeline::OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex)
{
	ColumnFillCoefficients[ColumnIndex] = FillCoefficient;
}

TSharedRef<ITableRow> SGeometryCacheTimeline::HandleTimelineListViewGenerateRow(TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SBox)
				[
					SNew(STextBlock)
						.Margin(FMargin{10,5})
						.Text(FText::FromString(*Text))
				]
		];
}

// FFrameRate::ComputeGridSpacing doesnt deal well with prime numbers, so we have a custom impl here
static bool ComputeGridSpacing(const FFrameRate& InFrameRate, float PixelsPerSecond, double& OutMajorInterval, int32& OutMinorDivisions, float MinTickPx, float DesiredMajorTickPx)
{
	// First try built-in spacing
	const bool bResult = InFrameRate.ComputeGridSpacing(PixelsPerSecond, OutMajorInterval, OutMinorDivisions, MinTickPx, DesiredMajorTickPx);
	if (!bResult || OutMajorInterval == 1.0)
	{
		if (PixelsPerSecond <= 0.f)
		{
			return false;
		}

		const int32 RoundedFPS = static_cast<int32>(FMath::RoundToInt(InFrameRate.AsDecimal()));

		if (RoundedFPS > 0)
		{
			// Showing frames
			TArray<int32, TInlineAllocator<10>> CommonBases;

			// Divide the rounded frame rate by 2s, 3s or 5s recursively
			{
				const int32 Denominators[] = { 2, 3, 5 };

				int32 LowestBase = RoundedFPS;
				for (;;)
				{
					CommonBases.Add(LowestBase);

					if (LowestBase % 2 == 0) { LowestBase = LowestBase / 2; }
					else if (LowestBase % 3 == 0) { LowestBase = LowestBase / 3; }
					else if (LowestBase % 5 == 0) { LowestBase = LowestBase / 5; }
					else
					{
						int32 LowestResult = LowestBase;
						for (int32 Denominator : Denominators)
						{
							int32 Result = LowestBase / Denominator;
							if (Result > 0 && Result < LowestResult)
							{
								LowestResult = Result;
							}
						}

						if (LowestResult < LowestBase)
						{
							LowestBase = LowestResult;
						}
						else
						{
							break;
						}
					}
				}
			}

			Algo::Reverse(CommonBases);

			const int32 Scale = static_cast<int32>(FMath::CeilToInt(DesiredMajorTickPx / PixelsPerSecond * InFrameRate.AsDecimal()));
			const int32 BaseIndex = FMath::Min(Algo::LowerBound(CommonBases, Scale), CommonBases.Num() - 1);
			const int32 Base = CommonBases[BaseIndex];

			const int32 MajorIntervalFrames = FMath::CeilToInt((float)Scale / static_cast<float>(Base)) * Base;
			OutMajorInterval = MajorIntervalFrames * InFrameRate.AsInterval();

			// Find the lowest number of divisions we can show that's larger than the minimum tick size
			OutMinorDivisions = 0;
			for (int32 DivIndex = 0; DivIndex < BaseIndex; ++DivIndex)
			{
				if (Base % CommonBases[DivIndex] == 0)
				{
					const int32 MinorDivisions = MajorIntervalFrames / CommonBases[DivIndex];
					if (OutMajorInterval / MinorDivisions * PixelsPerSecond >= MinTickPx)
					{
						OutMinorDivisions = MinorDivisions;
						break;
					}
				}
			}
		}
	}

	return OutMajorInterval != 0;
}

bool SGeometryCacheTimeline::GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const
{
	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FFrameRate DisplayRate = BindingAsset.Pin()->GetFrameRate();
	const double BiggestTime = ViewRange.Get().GetUpperBoundValue();
	const FString TickString = NumericTypeInterface->ToString((BiggestTime * DisplayRate).FrameNumber.Value);
	const FVector2D MaxTextSize = FontMeasureService->Measure(TickString, SmallLayoutFont);

	constexpr float MajorTickMultiplier = 2.f;

	const float MinTickPx = static_cast<float>(MaxTextSize.X) + 5.f;
	const float DesiredMajorTickPx = static_cast<float>(MaxTextSize.X) * MajorTickMultiplier;

	if (PhysicalWidth > 0 && DisplayRate.AsDecimal() > 0)
	{
		return ComputeGridSpacing(
			DisplayRate,
			static_cast<float>(PhysicalWidth / ViewRange.Get().Size<double>()),
			OutMajorInterval,
			OutMinorDivisions,
			MinTickPx,
			DesiredMajorTickPx);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE