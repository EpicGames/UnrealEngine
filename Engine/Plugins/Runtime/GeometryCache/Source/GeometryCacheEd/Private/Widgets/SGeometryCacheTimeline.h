// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ITimeSlider.h"
#include "AnimatedRange.h"
#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/NumericTypeInterface.h"

#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class FGeometryCacheTimeSlideController;
class UGeometryCacheComponent;

class FGeometryCacheTimelineBindingAsset;

class SGeometryCacheTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGeometryCacheTimeline) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FGeometryCacheTimelineBindingAsset>& InBindingAsset);

	/** Get numeric Type interface for converting between frame numbers and display formats. */
	TSharedRef<INumericTypeInterface<double>> GetNumericTypeInterface() const { return NumericTypeInterface.ToSharedRef(); }

	/** Compute a major grid interval and number of minor divisions to display */
	bool GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const;

private:
	void HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate) const;

	void InitTrackNames();

	float GetColumnFillCoefficient(int32 ColumnIndex) const
	{
		return ColumnFillCoefficients[ColumnIndex];
	}

	void OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex);

	TSharedRef<ITableRow> HandleTimelineListViewGenerateRow(TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable);
private:
	/** The fill coefficients of each column in the grid. */
	float ColumnFillCoefficients[2];
	
	TWeakPtr<FGeometryCacheTimelineBindingAsset> BindingAsset;

	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

	TSharedPtr<FGeometryCacheTimeSlideController> TimeSliderController;

	TSharedPtr<ITimeSlider> TimeSlider;

	TArray<TSharedPtr<FString>> TrackNames;

	TSharedPtr<SListView<TSharedPtr<FString> > > TracksListView;

	TAttribute<FAnimatedRange> ViewRange;
};