// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCacheTimelineOverlay.h"

#include "GeometryCacheTimeSliderController.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"

class FSlateRect;
class FWidgetStyle;
struct FGeometry;

int32 SGeometryCacheTimelineOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FPaintViewAreaArgs PaintArgs;
	PaintArgs.bDisplayTickLines = bDisplayTickLines.Get();
	PaintArgs.bDisplayScrubPosition = bDisplayScrubPosition.Get();

	if (PaintPlaybackRangeArgs.IsSet())
	{
		PaintArgs.PlaybackRangeArgs = PaintPlaybackRangeArgs.Get();
	}

	TimeSliderController->OnPaintViewArea(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, ShouldBeEnabled(bParentEnabled), PaintArgs);

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

