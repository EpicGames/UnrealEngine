// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCacheTimelineSplitterOverlay.h"

#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

struct FCaptureLostEvent;
struct FPointerEvent;

void SGeometryCacheTimelineSplitterOverlay::Construct( const FArguments& InArgs )
{
	SetVisibility(EVisibility::SelfHitTestInvisible);

	Splitter = SArgumentNew(InArgs, SSplitter);
	Splitter->SetVisibility(EVisibility::HitTestInvisible);
	AddSlot()
	[
		Splitter.ToSharedRef()
	];

	for (int32 Index = 0; Index < Splitter->GetChildren()->Num() - 1; ++Index)
	{
		AddSlot()
		.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SGeometryCacheTimelineSplitterOverlay::GetSplitterHandlePadding, Index)))
		[
			SNew(SBox)
			.Visibility(EVisibility::Visible)
		];
	}
}

void SGeometryCacheTimelineSplitterOverlay::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	FArrangedChildren SplitterChildren(ArrangedChildren.GetFilter());
	Splitter->ArrangeChildren(AllottedGeometry, SplitterChildren);

	SlotPadding.Reset();

	for (int32 Index = 0; Index < SplitterChildren.Num() - 1; ++Index)
	{
		const FGeometry& ThisGeometry = SplitterChildren[Index].Geometry;
		const FGeometry& NextGeometry = SplitterChildren[Index + 1].Geometry;

		if (Splitter->GetOrientation() == EOrientation::Orient_Horizontal)
		{
			SlotPadding.Add(FMargin(
				ThisGeometry.Position.X + static_cast<float>(ThisGeometry.GetLocalSize().X),
				0,
				static_cast<float>(AllottedGeometry.Size.X) - NextGeometry.Position.X,
				0)
			);
		}
		else
		{
			SlotPadding.Add(FMargin(
				0,
				ThisGeometry.Position.Y + static_cast<float>(ThisGeometry.GetLocalSize().Y),
				0,
				static_cast<float>(AllottedGeometry.Size.Y) - NextGeometry.Position.Y)
			);
		}
	}

	SOverlay::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

FMargin SGeometryCacheTimelineSplitterOverlay::GetSplitterHandlePadding(int32 Index) const
{
	if (SlotPadding.IsValidIndex(Index))
	{
		return SlotPadding[Index];
	}

	return 0.f;
}
	
FCursorReply SGeometryCacheTimelineSplitterOverlay::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return Splitter->OnCursorQuery(MyGeometry, CursorEvent);
}

FReply SGeometryCacheTimelineSplitterOverlay::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = Splitter->OnMouseButtonDown(MyGeometry, MouseEvent);
	if (Reply.GetMouseCaptor().IsValid())
	{
		// Set us to be the mouse captor so we can forward events properly
		Reply.CaptureMouse( SharedThis(this) );
		SetVisibility(EVisibility::Visible);
	}
	return Reply;
}
	
void SGeometryCacheTimelineSplitterOverlay::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SetVisibility(EVisibility::SelfHitTestInvisible);
	SOverlay::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SGeometryCacheTimelineSplitterOverlay::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = Splitter->OnMouseButtonUp(MyGeometry, MouseEvent);
	if (Reply.ShouldReleaseMouse())
	{
		SetVisibility(EVisibility::SelfHitTestInvisible);
	}
	return Reply;
}

FReply SGeometryCacheTimelineSplitterOverlay::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return Splitter->OnMouseMove(MyGeometry, MouseEvent);
}

void SGeometryCacheTimelineSplitterOverlay::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	return Splitter->OnMouseLeave(MouseEvent);
}

