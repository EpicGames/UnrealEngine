// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGeometryCacheTimelineTransportControls.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheTimelineBindingAsset.h"
#include "Modules/ModuleManager.h"
#include "STimelineCustomTransportControl.h"

void SGeometryCacheTimelineTransportControls::Construct(const FArguments& InArgs, const TSharedRef<FGeometryCacheTimelineBindingAsset>& InBindingAsset)
{
	WeakBindingAsset = InBindingAsset;

	FGeometryCacheTimelineTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnForwardPlay = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_Forward);
	TransportControlArgs.OnBackwardPlay = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_Backward);
	TransportControlArgs.OnForwardStep = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_Forward_Step);
	TransportControlArgs.OnBackwardStep = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_Backward_Step);
	TransportControlArgs.OnForwardEnd = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_Forward_End);
	TransportControlArgs.OnBackwardEnd = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_Backward_End);
	TransportControlArgs.OnToggleLooping = FOnClicked::CreateSP(this, &SGeometryCacheTimelineTransportControls::OnClick_ToggleLoop);
	TransportControlArgs.OnGetLooping = FOnGetLooping::CreateSP(this, &SGeometryCacheTimelineTransportControls::IsLoopStatusOn);
	TransportControlArgs.OnGetPlaybackMode = FOnGetPlaybackMode::CreateSP(this, &SGeometryCacheTimelineTransportControls::GetPlaybackMode);
	
	ChildSlot
	[
		SNew(STimelineCustomTransportControl)
			.TransportArgs(TransportControlArgs)
	];
}

FReply SGeometryCacheTimelineTransportControls::OnClick_Forward_Step()
{
	if (WeakBindingAsset.IsValid())
	{
		WeakBindingAsset.Pin()->GetPreviewComponent()->StepForward();
	}
	return FReply::Handled();
}

FReply SGeometryCacheTimelineTransportControls::OnClick_Forward_End()
{
	if (WeakBindingAsset.IsValid())
	{
		WeakBindingAsset.Pin()->GetPreviewComponent()->ForwardEnd();
	}
	return FReply::Handled();
}

FReply SGeometryCacheTimelineTransportControls::OnClick_Backward_Step()
{
	if (WeakBindingAsset.IsValid())
	{
		WeakBindingAsset.Pin()->GetPreviewComponent()->StepBackward();
	}
	return FReply::Handled();
}

FReply SGeometryCacheTimelineTransportControls::OnClick_Backward_End()
{
	if (WeakBindingAsset.IsValid())
	{
		WeakBindingAsset.Pin()->GetPreviewComponent()->BackwardEnd();
	}
	return FReply::Handled();
}

FReply SGeometryCacheTimelineTransportControls::OnClick_Forward()
{
	if (WeakBindingAsset.IsValid())
	{
		TWeakObjectPtr<UGeometryCacheComponent> GeometryCacheComponent = WeakBindingAsset.Pin()->GetPreviewComponent();
		const bool bIsPlaying = GeometryCacheComponent->IsPlaying();
		const bool bIsPlayingReversed = GeometryCacheComponent->IsPlayingReversed();

		if (bIsPlayingReversed && bIsPlaying)
		{
			GeometryCacheComponent->Play();
		}
		else if (bIsPlaying)
		{
			GeometryCacheComponent->Stop();
		}
		else
		{
			if (GeometryCacheComponent->GetElapsedTime() >= GeometryCacheComponent->GetDuration())
			{
				GeometryCacheComponent->SetCurrentTime(0.0f);
			}

			GeometryCacheComponent->Play();
		}
	}
	return FReply::Handled();
}

FReply SGeometryCacheTimelineTransportControls::OnClick_Backward()
{
	if (WeakBindingAsset.IsValid())
	{
		TWeakObjectPtr<UGeometryCacheComponent> GeometryCacheComponent = WeakBindingAsset.Pin()->GetPreviewComponent();
		const bool bIsPlaying = GeometryCacheComponent->IsPlaying();
		const bool bIsPlayingReversed = GeometryCacheComponent->IsPlayingReversed();

		if (!bIsPlayingReversed && bIsPlaying)
		{
			GeometryCacheComponent->PlayReversed();
		}
		else if(bIsPlaying)
		{
			GeometryCacheComponent->Stop();
		}
		else
		{
			if (GeometryCacheComponent->GetElapsedTime() <= 0.0f)
			{
				GeometryCacheComponent->SetCurrentTime(GeometryCacheComponent->GetDuration());
			}

			GeometryCacheComponent->PlayReversed();
		}
	}

	return FReply::Handled();
}

FReply SGeometryCacheTimelineTransportControls::OnClick_ToggleLoop()
{
	if (WeakBindingAsset.IsValid())
	{
		WeakBindingAsset.Pin()->GetPreviewComponent()->ToggleLooping();
	}
	
	return FReply::Handled();
}

bool SGeometryCacheTimelineTransportControls::IsLoopStatusOn() const
{
	if (WeakBindingAsset.IsValid())
	{
		return WeakBindingAsset.Pin()->GetPreviewComponent()->IsLooping();
	}
	return false;
}

EPlaybackMode::Type SGeometryCacheTimelineTransportControls::GetPlaybackMode() const
{
	if (WeakBindingAsset.IsValid())
	{
		TWeakObjectPtr<UGeometryCacheComponent> GeometryCacheComponent = WeakBindingAsset.Pin()->GetPreviewComponent();
		if (GeometryCacheComponent.IsValid())
		{
			if (GeometryCacheComponent->IsPlaying())
			{
				if (GeometryCacheComponent->IsPlayingReversed())
				{
					return EPlaybackMode::PlayingReverse;
				}

				return EPlaybackMode::PlayingForward;
			}
		}
	}
	
	return EPlaybackMode::Stopped;
}