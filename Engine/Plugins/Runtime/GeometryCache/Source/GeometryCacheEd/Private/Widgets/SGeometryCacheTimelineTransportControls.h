// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "ITransportControl.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FGeometryCacheTimelineBindingAsset;

class SGeometryCacheTimelineTransportControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGeometryCacheTimelineTransportControls) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FGeometryCacheTimelineBindingAsset>& InBindingAsset);

private:
	
	FReply OnClick_Forward_Step();

	FReply OnClick_Forward_End();

	FReply OnClick_Backward_Step();

	FReply OnClick_Backward_End();

	FReply OnClick_Forward();

	FReply OnClick_Backward();

	FReply OnClick_ToggleLoop();

	bool IsLoopStatusOn() const;

	EPlaybackMode::Type GetPlaybackMode() const;

private:
	TWeakPtr<FGeometryCacheTimelineBindingAsset> WeakBindingAsset;

	
};