// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimelineCustomTransportControl.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/EnumRange.h"
#include "SlateGlobals.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "STransportControlCustom"

namespace TransportControlConstants
{
	constexpr const int NumTransportControlButtons = 7;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> STimelineCustomTransportControl::MakeTransportControlWidget(ETransportControlWidgetType WidgetType, bool bAreButtonsFocusable, const FOnMakeTransportWidget& MakeCustomWidgetDelegate)
{
	switch (WidgetType)
	{
	case ETransportControlWidgetType::BackwardEnd:
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.OnClicked(TransportControlArgs.OnBackwardEnd)
			.Visibility(TransportControlArgs.OnBackwardEnd.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(LOCTEXT("ToFront", "To Front"))
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FAppStyle::Get().GetBrush("Animation.Backward_End"))
			];
	case ETransportControlWidgetType::BackwardStep:
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.OnClicked(TransportControlArgs.OnBackwardStep)
			.Visibility(TransportControlArgs.OnBackwardStep.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(LOCTEXT("ToPrevious", "To Previous"))
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FAppStyle::Get().GetBrush("Animation.Backward_Step"))
			];
	case ETransportControlWidgetType::BackwardPlay:
		return SAssignNew(BackwardPlayButton, SButton)
			.OnClicked(TransportControlArgs.OnBackwardPlay)
			.Visibility(TransportControlArgs.OnBackwardPlay.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(LOCTEXT("Reverse", "Reverse"))
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(this, &STimelineCustomTransportControl::GetBackwardStatusIcon)
			];
	case ETransportControlWidgetType::ForwardPlay:
		return SAssignNew(ForwardPlayButton, SButton)
			.OnClicked(TransportControlArgs.OnForwardPlay)
			.Visibility(TransportControlArgs.OnForwardPlay.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(this, &STimelineCustomTransportControl::GetForwardStatusTooltip)
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(this, &STimelineCustomTransportControl::GetForwardStatusIcon)
			];
	case ETransportControlWidgetType::ForwardStep:
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.OnClicked(TransportControlArgs.OnForwardStep)
			.Visibility(TransportControlArgs.OnForwardStep.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(LOCTEXT("ToNext", "To Next"))
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FAppStyle::Get().GetBrush("Animation.Forward_Step"))
			];
	case ETransportControlWidgetType::ForwardEnd:
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.OnClicked(TransportControlArgs.OnForwardEnd)
			.Visibility(TransportControlArgs.OnForwardEnd.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(LOCTEXT("ToEnd", "To End"))
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FAppStyle::Get().GetBrush("Animation.Forward_End"))
			];
	case ETransportControlWidgetType::Loop:
		return SAssignNew(LoopButton, SButton)
			.OnClicked(TransportControlArgs.OnToggleLooping)
			.Visibility(TransportControlArgs.OnGetLooping.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
			.ToolTipText(this, &STimelineCustomTransportControl::GetLoopStatusTooltip)
			.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
			.ContentPadding(0.0f)
			.IsFocusable(bAreButtonsFocusable)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(this, &STimelineCustomTransportControl::GetLoopStatusIcon)
			];
	default:
	{
		return SNullWidget::NullWidget;
	}
	}
}

void STimelineCustomTransportControl::Construct(const FArguments& InArgs)
{
	TransportControlArgs = InArgs._TransportArgs;

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::ClipToBounds);

	TArray<ETransportControlWidgetType, TFixedAllocator<TransportControlConstants::NumTransportControlButtons>> ButtonWidgetTypes = {
		ETransportControlWidgetType::BackwardEnd,
		ETransportControlWidgetType::BackwardStep,
		ETransportControlWidgetType::BackwardPlay,
		ETransportControlWidgetType::ForwardPlay,
		ETransportControlWidgetType::ForwardStep,
		ETransportControlWidgetType::ForwardEnd,
		ETransportControlWidgetType::Loop,
	};

	for (ETransportControlWidgetType WidgetType : ButtonWidgetTypes)
	{
		TSharedPtr<SWidget> Widget = MakeTransportControlWidget(WidgetType, InArgs._TransportArgs.bAreButtonsFocusable);
		if (Widget.IsValid())
		{
			HorizontalBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					Widget.ToSharedRef()
				];
		}
	}

	ChildSlot
		[
			HorizontalBox
		];
}

bool STimelineCustomTransportControl::IsTickable() const
{
	return false;
}

void STimelineCustomTransportControl::Tick(float DeltaTime)
{
}

const FSlateBrush* STimelineCustomTransportControl::GetForwardStatusIcon() const
{
	EPlaybackMode::Type PlaybackMode = EPlaybackMode::Stopped;
	if (TransportControlArgs.OnGetPlaybackMode.IsBound())
	{
		PlaybackMode = TransportControlArgs.OnGetPlaybackMode.Execute();
	}

	if (PlaybackMode == EPlaybackMode::PlayingForward)
	{
		return FAppStyle::Get().GetBrush("Animation.Pause");
	}

	return FAppStyle::Get().GetBrush("Animation.Forward");
}

FText STimelineCustomTransportControl::GetForwardStatusTooltip() const
{
	if (TransportControlArgs.OnGetPlaybackMode.IsBound() &&
		TransportControlArgs.OnGetPlaybackMode.Execute() == EPlaybackMode::PlayingForward)
	{
		return LOCTEXT("Pause", "Pause");
	}

	return LOCTEXT("Play", "Play");
}

const FSlateBrush* STimelineCustomTransportControl::GetBackwardStatusIcon() const
{
	if (TransportControlArgs.OnGetPlaybackMode.IsBound() &&
		TransportControlArgs.OnGetPlaybackMode.Execute() == EPlaybackMode::PlayingReverse)
	{
		return FAppStyle::Get().GetBrush("Animation.Pause");
	}

	return FAppStyle::Get().GetBrush("Animation.Backward");
}

const FSlateBrush* STimelineCustomTransportControl::GetLoopStatusIcon() const
{
	if (TransportControlArgs.OnGetLooping.IsBound() &&
		TransportControlArgs.OnGetLooping.Execute())
	{
		return FAppStyle::Get().GetBrush("Animation.Loop.Enabled");
	}

	return FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
}

FText STimelineCustomTransportControl::GetLoopStatusTooltip() const
{
	if (TransportControlArgs.OnGetLooping.IsBound() &&
		TransportControlArgs.OnGetLooping.Execute())
	{
		return LOCTEXT("Looping", "Looping");
	}

	return LOCTEXT("NoLooping", "No Looping");
}

EActiveTimerReturnType STimelineCustomTransportControl::TickPlayback(double InCurrentTime, float InDeltaTime)
{
	TransportControlArgs.OnTickPlayback.Execute(InCurrentTime, InDeltaTime);
	return EActiveTimerReturnType::Continue;
}

FReply STimelineCustomTransportControl::OnToggleLooping()
{
	return FReply::Handled();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE