// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "ITransportControl.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "TickableEditorObject.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

class FActiveTimerHandle;
class SButton;
struct FSlateBrush;

DECLARE_DELEGATE_RetVal(bool, FOnGetLooping)
DECLARE_DELEGATE_RetVal(EPlaybackMode::Type, FOnGetPlaybackMode)
DECLARE_DELEGATE_TwoParams(FOnTickPlayback, double /*InCurrentTime*/, float /*InDeltaTime*/)

struct FGeometryCacheTimelineTransportControlArgs
{
	FGeometryCacheTimelineTransportControlArgs()
		: OnForwardPlay()
		, OnBackwardPlay()
		, OnForwardStep()
		, OnBackwardStep()
		, OnForwardEnd()
		, OnBackwardEnd()
		, OnToggleLooping()
		, OnGetLooping()
		, OnGetPlaybackMode()
		, OnTickPlayback()
		, bAreButtonsFocusable(true)
	{}

	FOnClicked OnForwardPlay;
	FOnClicked OnBackwardPlay;
	FOnClicked OnForwardStep;
	FOnClicked OnBackwardStep;
	FOnClicked OnForwardEnd;
	FOnClicked OnBackwardEnd;
	FOnClicked OnToggleLooping;
	FOnGetLooping OnGetLooping;
	FOnGetPlaybackMode OnGetPlaybackMode;
	FOnTickPlayback OnTickPlayback;
	bool bAreButtonsFocusable;
};

class STimelineCustomTransportControl : public ITransportControl, public FTickableEditorObject
{
public:
	SLATE_BEGIN_ARGS(STimelineCustomTransportControl) 
		:_TransportArgs()
		{}
		SLATE_ARGUMENT(FGeometryCacheTimelineTransportControlArgs, TransportArgs)
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

	virtual ~STimelineCustomTransportControl() = default;

	using SWidget::Tick;

	// Begin FTickableObjectBase implementation
	virtual bool IsTickable() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(STimelineCustomTransportControl, STATGROUP_Tickables); }
	// End FTickableObjectBase

private:
	const FSlateBrush* GetForwardStatusIcon() const;
	FText GetForwardStatusTooltip() const;
	const FSlateBrush* GetBackwardStatusIcon() const;
	const FSlateBrush* GetLoopStatusIcon() const;
	FText GetLoopStatusTooltip() const;

	/** Executes the OnTickPlayback delegate */
	EActiveTimerReturnType TickPlayback(double InCurrentTime, float InDeltaTime);

	FReply OnToggleLooping();

	/** Make default transport control widgets */
	TSharedPtr<SWidget> MakeTransportControlWidget(ETransportControlWidgetType WidgetType, bool bAreButtonsFocusable, const FOnMakeTransportWidget& MakeCustomWidgetDelegate = FOnMakeTransportWidget());

private:
	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Whether the active timer is currently registered */
	bool bIsActiveTimerRegistered;

	FGeometryCacheTimelineTransportControlArgs TransportControlArgs;

	TSharedPtr<SButton> ForwardPlayButton;
	TSharedPtr<SButton> BackwardPlayButton;
	TSharedPtr<SButton> LoopButton;
};
