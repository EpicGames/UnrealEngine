// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrackEditor.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "SequencerSectionPainter.h"
#include "MovieSceneGeometryCacheTrack.h"
#include "MovieSceneGeometryCacheSection.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Fonts/FontMeasure.h"
#include "GeometryCacheComponent.h"
#include "GeometryCache.h"
#include "Styling/SlateIconFinder.h"
#include "LevelSequence.h"
#include "TimeToPixel.h"

namespace GeometryCacheEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	const uint32 AnimationTrackHeight = 28;
}

#define LOCTEXT_NAMESPACE "FGeometryCacheTrackEditor"

static UGeometryCacheComponent* AcquireGeometryCacheFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (UGeometryCacheComponent* GeometryMeshComp = Cast<UGeometryCacheComponent>(Component))
			{
				return GeometryMeshComp;
			}
		}
	}
	else if (UGeometryCacheComponent* GeometryMeshComp = Cast<UGeometryCacheComponent>(BoundObject))
	{
		if (GeometryMeshComp->GetGeometryCache())
		{
			return GeometryMeshComp;
		}
	}

	return nullptr;
}


FGeometryCacheSection::FGeometryCacheSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: Section(*CastChecked<UMovieSceneGeometryCacheSection>(&InSection))
	, Sequencer(InSequencer)
	, InitialFirstLoopStartOffsetDuringResize(0)
	, InitialStartTimeDuringResize(0)
{ }


UMovieSceneSection* FGeometryCacheSection::GetSectionObject()
{
	return &Section;
}


FText FGeometryCacheSection::GetSectionTitle() const
{
	if (Section.Params.GeometryCacheAsset != nullptr)
	{
		return FText::FromString(Section.Params.GeometryCacheAsset->GetName());
	
	}
	return LOCTEXT("NoGeometryCacheSection", "No GeometryCache");
}


float FGeometryCacheSection::GetSectionHeight() const
{
	return (float)GeometryCacheEditorConstants::AnimationTrackHeight;
}


int32 FGeometryCacheSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	int32 LayerId = Painter.PaintSectionBackground();

	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	if (!Section.HasStartFrame() || !Section.HasEndFrame())
	{
		return LayerId;
	}

	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	// Add lines where the animation starts and ends/loops
	const float AnimPlayRate = FMath::IsNearlyZero(Section.Params.PlayRate) ? 1.0f : Section.Params.PlayRate;
	const float Duration = Section.Params.GetSequenceLength();
	const float SeqLength = static_cast<float>(Duration - TickResolution.AsSeconds(Section.Params.StartFrameOffset + Section.Params.EndFrameOffset) / AnimPlayRate);
	const float FirstLoopSeqLength = static_cast<float>(SeqLength - TickResolution.AsSeconds(Section.Params.FirstLoopStartFrameOffset) / AnimPlayRate);

	if (!FMath::IsNearlyZero(SeqLength, KINDA_SMALL_NUMBER) && SeqLength > 0)
	{
		float MaxOffset = static_cast<float>(Section.GetRange().Size<FFrameTime>() / TickResolution);
		float OffsetTime = FirstLoopSeqLength;
		float StartTime = static_cast<float>(Section.GetInclusiveStartFrame() / TickResolution);

		while (OffsetTime < MaxOffset)
		{
			float OffsetPixel = TimeToPixelConverter.SecondsToPixel(StartTime + OffsetTime) - TimeToPixelConverter.SecondsToPixel(StartTime);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.SectionGeometry.MakeChild(
					FVector2D(2.f, Painter.SectionGeometry.Size.Y - 2.f),
					FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
				).ToPaintGeometry(),
				GenericDivider,
				DrawEffects
			);

			OffsetTime += SeqLength;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (Painter.bIsSelected && SequencerPtr.IsValid())
	{
		FFrameTime CurrentTime = SequencerPtr->GetLocalTime().Time;
		if (Section.GetRange().Contains(CurrentTime.FrameNumber) && Section.Params.GeometryCacheAsset != nullptr)
		{
			const float Time = TimeToPixelConverter.FrameToPixel(CurrentTime);

			UGeometryCache* GeometryCache = Section.Params.GeometryCacheAsset;

			// Draw the current time next to the scrub handle
			const float AnimTime = Section.MapTimeToAnimation(Duration, CurrentTime, TickResolution);
			int32 FrameTime = GeometryCache->GetFrameAtTime(AnimTime);
			FString FrameString = FString::FromInt(FrameTime);

			const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			static const float TextOffsetPx = 10.f;
			bool  bDrawLeft = (Painter.SectionGeometry.Size.X - Time) < (TextSize.X + 22.f) - TextOffsetPx;
			float TextPosition = bDrawLeft ? static_cast<float>(Time - TextSize.X - TextOffsetPx) : Time + TextOffsetPx;
			//handle mirrored labels
			const float MajorTickHeight = 9.0f;
			FVector2D TextOffset(TextPosition, Painter.SectionGeometry.Size.Y - (MajorTickHeight + TextSize.Y));

			const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());
			const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
			// draw time string

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId + 5,
				Painter.SectionGeometry.ToPaintGeometry(TextSize + 2.0f * BoxPadding, FSlateLayoutTransform(TextOffset - BoxPadding)),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);

			FSlateDrawElement::MakeText(
				Painter.DrawElements,
				LayerId + 6,
				Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				DrawEffects,
				DrawColor
			);

		}
	}

	return LayerId;
}

void FGeometryCacheSection::BeginResizeSection()
{
	InitialFirstLoopStartOffsetDuringResize = Section.Params.FirstLoopStartFrameOffset;
	InitialStartTimeDuringResize = Section.HasStartFrame() ? Section.GetInclusiveStartFrame() : 0;
}

void FGeometryCacheSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	// Adjust the start offset when resizing from the beginning
	if (ResizeMode == SSRM_LeadingEdge)
	{
		FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber StartOffset = FrameRate.AsFrameNumber((ResizeTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

		StartOffset += InitialFirstLoopStartOffsetDuringResize;

		if (StartOffset < 0)
		{
			// Ensure start offset is not less than 0 and adjust ResizeTime
			ResizeTime = ResizeTime - StartOffset;

			StartOffset = FFrameNumber(0);
		}
		else
		{
			// If the start offset exceeds the length of one loop, trim it back.
			const FFrameNumber SeqLength = FrameRate.AsFrameNumber(Section.Params.GetSequenceLength()) - Section.Params.StartFrameOffset - Section.Params.EndFrameOffset;
			StartOffset = StartOffset % SeqLength;
		}

		Section.Params.FirstLoopStartFrameOffset = StartOffset;
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FGeometryCacheSection::BeginSlipSection()
{
	BeginResizeSection();
}

void FGeometryCacheSection::SlipSection(FFrameNumber SlipTime)
{
	FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameNumber StartOffset = FrameRate.AsFrameNumber((SlipTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

	StartOffset += InitialFirstLoopStartOffsetDuringResize;

	if (StartOffset < 0)
	{
		// Ensure start offset is not less than 0 and adjust ResizeTime
		SlipTime = SlipTime - StartOffset;

		StartOffset = FFrameNumber(0);
	}
	else
	{
		// If the start offset exceeds the length of one loop, trim it back.
		const FFrameNumber SeqLength = FrameRate.AsFrameNumber(Section.Params.GetSequenceLength()) - Section.Params.StartFrameOffset - Section.Params.EndFrameOffset;
		StartOffset = StartOffset % SeqLength;
	}

	Section.Params.FirstLoopStartFrameOffset = StartOffset;

	ISequencerSection::SlipSection(SlipTime);
}

void FGeometryCacheSection::BeginDilateSection()
{
	Section.PreviousPlayRate = Section.Params.PlayRate; //make sure to cache the play rate
}
void FGeometryCacheSection::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	Section.Params.PlayRate = Section.PreviousPlayRate / DilationFactor;
	Section.SetRange(NewRange);
}

FGeometryCacheTrackEditor::FGeometryCacheTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ }


TSharedRef<ISequencerTrackEditor> FGeometryCacheTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FGeometryCacheTrackEditor(InSequencer));
}


bool FGeometryCacheTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneGeometryCacheTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FGeometryCacheTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneGeometryCacheTrack::StaticClass();
}


TSharedRef<ISequencerSection> FGeometryCacheTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FGeometryCacheSection(SectionObject, GetSequencer()));
}

FText FGeometryCacheTrackEditor::GetDisplayName() const
{
	return LOCTEXT("GeometryCacheTrackEditor_DisplayName", "Geometry");
}

void FGeometryCacheTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UGeometryCacheComponent::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		UGeometryCacheComponent* GeomMeshComp = AcquireGeometryCacheFromObjectGuid(ObjectBindings[0], GetSequencer());

		if (GeomMeshComp)
		{
			UMovieSceneTrack* Track = nullptr;

			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("Sequencer", "AddGeometryCache", "Geometry Cache"),
				NSLOCTEXT("Sequencer", "AddGeometryCacheTooltip", "Adds a Geometry Cache track."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FGeometryCacheTrackEditor::BuildGeometryCacheTrack, ObjectBindings, Track)
				)
			);
		}
	}
}

void FGeometryCacheTrackEditor::BuildGeometryCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SequencerPtr.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("AddGeometryCache_Transaction", "Add Geometry Cache"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			if (ObjectBinding.IsValid())
			{
				UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);

				UGeometryCacheComponent* GeomMeshComp = AcquireGeometryCacheFromObjectGuid(ObjectBinding, GetSequencer());

				if (Object && GeomMeshComp)
				{
					AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FGeometryCacheTrackEditor::AddKeyInternal, Object, GeomMeshComp, Track));
				}
			}
		}
	}
}

FKeyPropertyResult FGeometryCacheTrackEditor::AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UGeometryCacheComponent* GeomCacheComp, UMovieSceneTrack* Track)
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		if (!Track)
		{
			Track = AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectHandle, UMovieSceneGeometryCacheTrack::StaticClass(), NAME_None);
			KeyPropertyResult.bTrackCreated = true;
		}

		if (ensure(Track))
		{
			Track->Modify();

			UMovieSceneSection* NewSection = Cast<UMovieSceneGeometryCacheTrack>(Track)->AddNewAnimation(KeyTime, GeomCacheComp);
			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}

	return KeyPropertyResult;
}


TSharedPtr<SWidget> FGeometryCacheTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UGeometryCacheComponent* GeomMeshComp = AcquireGeometryCacheFromObjectGuid(ObjectBinding, GetSequencer());

	if (GeomMeshComp)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();

		auto SubMenuCallback = [this, ObjectBinding, Track]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			TArray<FGuid> ObjectBindings;
			ObjectBindings.Add(ObjectBinding);

			BuildGeometryCacheTrack(ObjectBindings, Track);
			
			return MenuBuilder.MakeWidget();
		};

		return UE::Sequencer::MakeAddButton(LOCTEXT("GeomCacheText", "Geometry Cache"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}

}

const FSlateBrush* FGeometryCacheTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(UGeometryCacheComponent::StaticClass()).GetIcon();
}


#undef LOCTEXT_NAMESPACE
