// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "MovieSceneTrackEditor.h"

#define UE_API GEOMETRYCACHESEQUENCER_API

struct FBuildEditWidgetParams;

struct FAssetData;
class FMenuBuilder;
class FSequencerSectionPainter;
class UMovieSceneGeometryCacheSection;
class UMovieSceneSequence;
class UGeometryCacheComponent;

/**
 * Tools for Geometry Cache tracks
 */
class FGeometryCacheTrackEditor : public FMovieSceneTrackEditor
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	UE_API FGeometryCacheTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FGeometryCacheTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

private:

	UE_API void BuildGeometryCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	UE_API FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UGeometryCacheComponent* GeomCacheComp, UMovieSceneTrack* Track);
};


/** Class for animation sections */
class FGeometryCacheSection
	: public ISequencerSection
	, public TSharedFromThis<FGeometryCacheSection>
{
public:

	/** Constructor. */
	FGeometryCacheSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FGeometryCacheSection() = default;

public:

	// ISequencerSection interface

	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight() const override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	virtual void BeginResizeSection() override;
	virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	virtual void BeginSlipSection() override;
	virtual void SlipSection(FFrameNumber SlipTime) override;
	virtual void BeginDilateSection() override;
	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override;


private:

	/** The section we are visualizing */
	UMovieSceneGeometryCacheSection& Section;

	/** Used to draw animation frame, need selection state and local time*/
	TWeakPtr<ISequencer> Sequencer;

	/** Cached first loop start offset value valid only during resize */
	FFrameNumber InitialFirstLoopStartOffsetDuringResize;
	
	/** Cached start time valid only during resize */
	FFrameNumber InitialStartTimeDuringResize;
};

#undef UE_API
