// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class GeometryCacheSequencer : ModuleRules
    {
        public GeometryCacheSequencer(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicIncludePathModuleNames.AddRange(
                new string[] {
                "Sequencer",
                }
            );
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
					"LevelSequence",
                    "MovieScene",
                    "MovieSceneTracks",
                    "SequencerCore",
                    "Sequencer",
                    "Slate",
                    "SlateCore",
                    "UnrealEd",
                    "GeometryCacheTracks",
                    "GeometryCache"
                }
            );

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
