// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCacheSequencerModule.h"

#define LOCTEXT_NAMESPACE "GeometryCacheSequencer"



IMPLEMENT_MODULE(FGeometryCacheSequencerModule, GeometryCacheSequencer);

#undef LOCTEXT_NAMESPACE
