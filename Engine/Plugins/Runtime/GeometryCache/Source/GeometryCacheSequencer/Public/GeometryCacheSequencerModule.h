// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCacheModule.h"
#include "GeometryCacheTrackEditor.h"
#include "ISequencerModule.h"

/**
 * The public interface to this module
 */
class FGeometryCacheSequencerModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(GeometryCache);

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		TrackEditorBindingHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FGeometryCacheTrackEditor::CreateTrackEditor));

	}

	virtual void ShutdownModule() override
	{

		ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModulePtr)
		{
			SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
		}

	}

	FDelegateHandle TrackEditorBindingHandle;
};
