// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheStreamBase.h"
#include "Async/Async.h"
#include "GeometryCacheMeshData.h"
#include "Misc/ScopeRWLock.h"

enum class EStreamReadRequestStatus
{
	Scheduled,
	Completed,
	Cancelled
};

struct FGeometryCacheStreamReadRequest
{
	FGeometryCacheMeshData* MeshData = nullptr;
	int32 ReadIndex = 0;
	int32 FrameIndex = 0;
	EStreamReadRequestStatus Status = EStreamReadRequestStatus::Scheduled;
};

FGeometryCacheStreamBase::FGeometryCacheStreamBase(int32 ReadConcurrency, FGeometryCacheStreamDetails&& InDetails)
: Details(InDetails)
, CurrentFrameIndex(0)
, MaxCachedFrames(0)
, MaxCachedDuration(0.f)
, MaxMemAllowed(TNumericLimits<float>::Max())
, MemoryUsed(0.f)
, bCancellationRequested(false)
, bCacheNeedsUpdate(false)
{
	ensureMsgf(Details.SecondsPerFrame > 0.f, TEXT("GeometryCache stream was initialized with %f seconds per frame, which is invalid. Falling back to 1/24"), Details.SecondsPerFrame);
	if (Details.SecondsPerFrame <= 0.f)
	{
		Details.SecondsPerFrame = 1.f / 24.f;
	}

	for (int32 Index = 0; Index < ReadConcurrency; ++Index)
	{
		// Populate the ReadIndices. Note that it used as a stack
		ReadIndices.Push(Index);

		// Populate pool of reusable ReadRequests
		ReadRequestsPool.Add(new FGeometryCacheStreamReadRequest());
	}
}

FGeometryCacheStreamBase::~FGeometryCacheStreamBase()
{
	CancelRequests();

	// Delete all the cached MeshData
	FReadScopeLock ReadLock(FramesAvailableLock);
	for (const auto& Pair : FramesAvailable)
	{
		FGeometryCacheMeshData* MeshData = Pair.Value;
		DecrementMemoryStat(*MeshData);
		delete MeshData;
	}

	// And ReadRequests from the pool
	for (int32 Index = 0; Index < ReadRequestsPool.Num(); ++Index)
	{
		delete ReadRequestsPool[Index];
	}
}

int32 FGeometryCacheStreamBase::CancelRequests()
{
	TGuardValue<std::atomic<bool>, bool> CancellationRequested(bCancellationRequested, true);

	// Clear the FramesNeeded to prevent scheduling further reads
	FramesNeeded.Empty();

	// Wait for all read requests to complete
	TArray<int32> CompletedFrames;
	while (FramesRequested.Num())
	{
		UpdateRequestStatus(CompletedFrames);
		if (FramesRequested.Num())
		{
			FPlatformProcess::Sleep(0.01f);
		}
	}

	return CompletedFrames.Num();
}

bool FGeometryCacheStreamBase::RequestFrameData()
{
	check(IsInGameThread());

	if (FramesNeeded.Num() == 0)
	{
		return false;
	}

	FReadScopeLock ReadLock(FramesAvailableLock);

	// Get the next frame index to read that is not already available and not in flight
	int32 FrameIndex = FramesNeeded[0];
	bool bFrameIndexValid = false;
	while (FramesNeeded.Num() && !bFrameIndexValid)
	{
		FrameIndex = FramesNeeded[0];
		bFrameIndexValid = true;
		if (FramesAvailable.Contains(FrameIndex))
		{
			FramesNeeded.Remove(FrameIndex);
			bFrameIndexValid = false;
		}
		for (const FGeometryCacheStreamReadRequest* Request : FramesRequested)
		{
			if (Request->FrameIndex == FrameIndex)
			{
				FramesNeeded.Remove(FrameIndex);
				bFrameIndexValid = false;
				break;
			}
		}
	}

	if (!bFrameIndexValid)
	{
		return false;
	}

	if (ReadIndices.Num() > 0)
	{
		PrepareRead();

		// Get any ReadIndex available
		const int32 ReadIndex = ReadIndices.Pop(EAllowShrinking::No);

		// Take the ReadRequest from the pool at ReadIndex and initialize it
		FGeometryCacheStreamReadRequest*& ReadRequest = ReadRequestsPool[ReadIndex];
		ReadRequest->FrameIndex = FrameIndex;
		ReadRequest->ReadIndex = ReadIndex;
		ReadRequest->MeshData = new FGeometryCacheMeshData;
		ReadRequest->Status = EStreamReadRequestStatus::Scheduled;

		// Change the frame status from needed to requested
		FramesNeeded.Remove(FrameIndex);
		FramesRequested.Add(ReadRequest);

		// Schedule asynchronous read of the MeshData
		Async(
#if WITH_EDITOR
			EAsyncExecution::LargeThreadPool,
#else
			EAsyncExecution::ThreadPool,
#endif // WITH_EDITOR
			[this, ReadRequest]()
			{
				if (!bCancellationRequested)
				{
					checkSlow(ReadRequest->MeshData);
					GetMeshData(ReadRequest->FrameIndex, ReadRequest->ReadIndex, *ReadRequest->MeshData);
					ReadRequest->Status = EStreamReadRequestStatus::Completed;
				}
				else
				{
					ReadRequest->Status = EStreamReadRequestStatus::Cancelled;
				}
			});

		return true;
	}
	return false;
}

void FGeometryCacheStreamBase::UpdateRequestStatus(TArray<int32>& OutFramesCompleted)
{
	check(IsInGameThread());

	// Check if the cache needs to be updated either because the frame has advanced or because there's a new memory limit
	if (bCacheNeedsUpdate)
	{
		bCacheNeedsUpdate = false;
		UpdateFramesNeeded(CurrentFrameIndex, MaxCachedFrames);

		// Remove the unneeded frames, those that are available but don't need to be cached
		TArray<int32> UnneededFrames;
		{
			FReadScopeLock ReadLock(FramesAvailableLock);
			for (const auto& Pair : FramesAvailable)
			{
				int32 FrameIndex = Pair.Key;
				if (!FramesToBeCached.Contains(FrameIndex))
				{
					FGeometryCacheMeshData* MeshData = Pair.Value;
					DecrementMemoryStat(*MeshData);
					delete MeshData;

					UnneededFrames.Add(FrameIndex);
				}
			}
		}

		{
			FWriteScopeLock WriteLock(FramesAvailableLock);
			for (int32 FrameIndex : UnneededFrames)
			{
				FramesAvailable.Remove(FrameIndex);
			}
		}
	}

	FWriteScopeLock WriteLock(FramesAvailableLock);

	// Check the completion status of the read requests in progress
	TArray<FGeometryCacheStreamReadRequest*> CompletedRequests;
	for (FGeometryCacheStreamReadRequest* ReadRequest : FramesRequested)
	{
		// A cancelled read is still considered completed, it just hasn't read any data
		if (ReadRequest->Status == EStreamReadRequestStatus::Completed ||
			ReadRequest->Status == EStreamReadRequestStatus::Cancelled)
		{
			// Queue for removal after iterating
			CompletedRequests.Add(ReadRequest);

			// A cancelled read has an allocated mesh data that needs to be deleted
			bool bDeleteMeshData = ReadRequest->Status == EStreamReadRequestStatus::Cancelled;
			if (ReadRequest->Status == EStreamReadRequestStatus::Completed)
			{
				checkSlow(ReadRequest->MeshData);
				FResourceSizeEx ResSize;
				ReadRequest->MeshData->GetResourceSizeEx(ResSize);
				float FrameDataSize = float(ResSize.GetTotalMemoryBytes()) / (1024 * 1024);

				if (MemoryUsed + FrameDataSize < MaxMemAllowed)
				{
					if (!FramesAvailable.Contains(ReadRequest->FrameIndex))
					{
						// Cache result of read for retrieval later
						FramesAvailable.Add(ReadRequest->FrameIndex, ReadRequest->MeshData);
						IncrementMemoryStat(*ReadRequest->MeshData);
					}
					else
					{
						// The requested frame was already available, just delete it
						bDeleteMeshData = true;
					}
				}
				else
				{
					// The frame doesn't fit in the allowed memory budget so delete it
					// It should be requested again later if it's still needed
					bDeleteMeshData = true;
				}
			}

			// Push back the ReadIndex for reuse
			ReadIndices.Push(ReadRequest->ReadIndex);

			// Output the completed frame
			OutFramesCompleted.Add(ReadRequest->FrameIndex);

			if (bDeleteMeshData)
			{
				delete ReadRequest->MeshData;
				ReadRequest->MeshData = nullptr;
			}
		}
	}

	for (FGeometryCacheStreamReadRequest* ReadRequest : CompletedRequests)
	{
		FramesRequested.Remove(ReadRequest);
	}
}

void FGeometryCacheStreamBase::UpdateFramesNeeded(int32 StartFrameIndex, int32 NumFrames)
{
	FramesToBeCached.Empty(NumFrames);
	FramesNeeded.Empty(NumFrames);

	const int32 StartIndex = Details.StartFrameIndex;
	const int32 EndIndex = Details.EndFrameIndex;

	// FramesToBeCached are the frame indices that are required for playback, available or not,
	// while FramesNeeded are the frame indices that are not available yet so they need to be read
	auto AddFrameIndex = [this, &NumFrames](int32 FrameIndex)
	{
		FramesToBeCached.Add(FrameIndex);
		if (!FramesAvailable.Contains(FrameIndex))
		{
			FramesNeeded.Add(FrameIndex);
		}
		--NumFrames;
	};

	FReadScopeLock ReadLock(FramesAvailableLock);

	StartFrameIndex = FMath::Clamp(StartFrameIndex, StartIndex, EndIndex);

	// Also reserve space for the frame before start since playback is double-buffered
	int PreviousFrameIndex = FMath::Clamp(StartFrameIndex - 1, StartIndex, EndIndex);
	if (PreviousFrameIndex != StartFrameIndex)
	{
		--NumFrames;
	}

	// Populate the list of frame indices from given StartFrameIndex up to NumFrames or EndIndex
	for (int32 Index = StartFrameIndex; NumFrames > 0 && Index < EndIndex + 1; ++Index)
	{
		AddFrameIndex(Index);
	}

	// End of the range might have been reached before the requested NumFrames so add the remaining frames starting from StartIndex
	for (int32 Index = StartIndex; NumFrames > 0 && Index < PreviousFrameIndex; ++Index)
	{
		AddFrameIndex(Index);
	}

	// Frame before start is added at the end to preserve the priority of the other frames
	if (PreviousFrameIndex != StartFrameIndex)
	{
		AddFrameIndex(PreviousFrameIndex);
	}
}

void FGeometryCacheStreamBase::Prefetch(int32 StartFrameIndex, int32 NumFrames)
{
	const int32 MaxNumFrames = Details.NumFrames;

	// Validate the number of frames to be loaded
	if (NumFrames == 0)
	{
		// If no value specified, load the whole stream
		NumFrames = MaxNumFrames;
	}
	else
	{
		NumFrames = FMath::Clamp(NumFrames, 1, MaxNumFrames);
	}

	MaxCachedFrames = NumFrames;

	UpdateFramesNeeded(StartFrameIndex, NumFrames);

	if (FramesNeeded.Num() > 0)
	{
		// Force the first frame to be loaded and ready for retrieval
		LoadFrameData(FramesNeeded[0]);
		FramesNeeded.RemoveAt(0);
	}
}

uint32 FGeometryCacheStreamBase::GetNumFramesNeeded()
{
	return FramesNeeded.Num();
}

bool FGeometryCacheStreamBase::GetFrameData(int32 FrameIndex, FGeometryCacheMeshData& OutMeshData)
{
	// This function can be called from the render thread
	FReadScopeLock ReadLock(FramesAvailableLock);
	if (FGeometryCacheMeshData** MeshDataPtr = FramesAvailable.Find(FrameIndex))
	{
		OutMeshData = **MeshDataPtr;
		return true;
	}
	return false;
}

void FGeometryCacheStreamBase::LoadFrameData(int32 FrameIndex)
{
	check(IsInGameThread());

	FWriteScopeLock WriteLock(FramesAvailableLock);
	if (FramesAvailable.Contains(FrameIndex))
	{
		return;
	}

	PrepareRead();

	FGeometryCacheMeshData* MeshData = new FGeometryCacheMeshData;
	GetMeshData(FrameIndex, 0, *MeshData);
	FramesAvailable.Add(FrameIndex, MeshData);
	IncrementMemoryStat(*MeshData);
}

const FGeometryCacheStreamStats& FGeometryCacheStreamBase::GetStreamStats() const
{
	check(IsInGameThread());

	const int32 NumFrames = FramesAvailable.Num();
	const float Secs = Details.SecondsPerFrame * static_cast<float>(NumFrames);

	Stats.NumCachedFrames = NumFrames;
	Stats.CachedDuration = Secs;
	Stats.MemoryUsed = MemoryUsed;
	Stats.AverageBitrate = MemoryUsed / Secs;
	return Stats;
}

void FGeometryCacheStreamBase::SetLimits(float InMaxMemoryAllowed, float InMaxCachedDuration)
{
	check(IsInGameThread());

	if (!FMath::IsNearlyEqual(InMaxMemoryAllowed, MaxMemAllowed) ||
		!FMath::IsNearlyEqual(InMaxCachedDuration, MaxCachedDuration))
	{
		bCacheNeedsUpdate = true;
		MaxMemAllowed = InMaxMemoryAllowed;

		MaxCachedDuration = FMath::Min(InMaxCachedDuration, Details.Duration);
		MaxCachedFrames = FMath::Min(FMath::RoundToInt(MaxCachedDuration / Details.SecondsPerFrame), Details.NumFrames);
	}
}

void FGeometryCacheStreamBase::IncrementMemoryStat(const FGeometryCacheMeshData& MeshData)
{
	FResourceSizeEx ResSize;
	MeshData.GetResourceSizeEx(ResSize);

	MemoryUsed += static_cast<float>(ResSize.GetTotalMemoryBytes()) / (1024.0f * 1024.0f);
}

void FGeometryCacheStreamBase::DecrementMemoryStat(const FGeometryCacheMeshData& MeshData)
{
	FResourceSizeEx ResSize;
	MeshData.GetResourceSizeEx(ResSize);

	MemoryUsed -= static_cast<float>(ResSize.GetTotalMemoryBytes()) / (1024.0f * 1024.0f);
}

void FGeometryCacheStreamBase::UpdateCurrentFrameIndex(int32 FrameIndex)
{
	if (FrameIndex != CurrentFrameIndex)
	{
		const int32 FrameDelta = FMath::Abs(FrameIndex - CurrentFrameIndex);
		if (FrameDelta >= MaxCachedFrames && FramesNeeded.Num() == 0)
		{
			// When the time jump is greater than the number of cached frames and it is not currently streaming,
			// force loading the requested FrameIndex so that it gets displayed when the proxy is updated.
			LoadFrameData(FrameIndex);
		}
		CurrentFrameIndex = FrameIndex;
		bCacheNeedsUpdate = true;
	}
}
