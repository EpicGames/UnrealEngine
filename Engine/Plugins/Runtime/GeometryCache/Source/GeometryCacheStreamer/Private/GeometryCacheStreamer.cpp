// Copyright Epic Games, Inc. All Rights Reserved.

#include "IGeometryCacheStreamer.h"
#include "Containers/Ticker.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCacheModule.h"
#include "GeometryCacheStreamerSettings.h"
#include "IGeometryCacheStream.h"
#include "Templates/RefCounting.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "GeometryCacheStreamer"

DECLARE_DWORD_COUNTER_STAT(TEXT("GeometryCache Streams: Count"), STAT_GeometryCacheStream_Count, STATGROUP_GeometryCache);
DECLARE_FLOAT_COUNTER_STAT(TEXT("GeometryCache Streams: Look-Ahead (seconds)"), STAT_GeometryCacheStream_LookAhead, STATGROUP_GeometryCache);
DECLARE_FLOAT_COUNTER_STAT(TEXT("GeometryCache Streams: Total Bitrate (MB/s)"), STAT_GeometryCacheStream_Bitrate, STATGROUP_GeometryCache);
DECLARE_MEMORY_STAT(TEXT("GeometryCache Streams: Memory Used"), STAT_GeometryCacheStream_MemoryUsed, STATGROUP_GeometryCache);

static bool GShowGeometryCacheStreamerNotification = true;
static FAutoConsoleVariableRef CVarGeometryCacheStreamerShowNotification(
	TEXT("GeometryCache.Streamer.ShowNotification"),
	GShowGeometryCacheStreamerNotification,
	TEXT("Show notification while the GeometryCache streamer is streaming data"));

static bool GGeoCacheStreamerBlockTillFinishStreaming = false;
static FAutoConsoleVariableRef CVarBlockTillFinishStreaming(
	TEXT("GeometryCache.Streamer.BlockTillFinishStreaming"),
	GGeoCacheStreamerBlockTillFinishStreaming,
	TEXT("Force the GeometryCache streamer to block until it has finished streaming all the requested frames"));

class FGeometryCacheStreamer : public IGeometryCacheStreamer
{
public:
	FGeometryCacheStreamer();
	virtual ~FGeometryCacheStreamer();

	void Tick(float Time);

	//~ Begin IGeometryCacheStreamer Interface
	static IGeometryCacheStreamer& Get();

	virtual void RegisterTrack(UGeometryCacheTrack* Track, IGeometryCacheStream* Stream) override;
	virtual void UnregisterTrack(UGeometryCacheTrack* Track) override;
	virtual bool IsTrackRegistered(UGeometryCacheTrack* Track) const override;
	virtual bool TryGetFrameData(UGeometryCacheTrack* Track, int32 FrameIndex, FGeometryCacheMeshData& OutMeshData) override;
	//~ End IGeometryCacheStreamer Interface

private:
	void BalanceStreams();

private:
	FTSTicker::FDelegateHandle TickHandle;

	class FRefCountedStream : public UE::Private::FQueryableRefCountedObject
	{
	public:
		FRefCountedStream(IGeometryCacheStream* InStream)
		: Stream(InStream)
		{
			AddRef();
		}

		IGeometryCacheStream* Stream = nullptr;
	};

	typedef TMap<UGeometryCacheTrack*, FRefCountedStream*> FTracksToStreams;
	FTracksToStreams TracksToStreams;

	const int32 MaxReads;
	int32 NumReads;
	int32 CurrentIndex;

	TSharedPtr<SNotificationItem> StreamingNotification;
};

FGeometryCacheStreamer::FGeometryCacheStreamer()
: MaxReads(FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads()))
, NumReads(0)
, CurrentIndex(0)

{
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([this](float Time)
		{
			this->Tick(Time);
			return true;
		})
	);
}

FGeometryCacheStreamer::~FGeometryCacheStreamer()
{
	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}

void FGeometryCacheStreamer::Tick(float Time)
{
	int32 NumStreams = 0;
	int32 NumFramesToStream = 0;
	bool bWaitForFinish = false;
	do 
	{
		// The sole purpose of the Streamer is to schedule the streams for read at every engine loop
		BalanceStreams();

		// First step is to process the results from the previously scheduled read requests
		// to figure out the number of reads still in progress
		NumFramesToStream = 0;
		for (const auto& Pair : TracksToStreams)
		{
			TArray<int32> FramesCompleted;
			IGeometryCacheStream* Stream = Pair.Value->Stream;
			Stream->UpdateRequestStatus(FramesCompleted);
			NumReads -= FramesCompleted.Num();
			NumFramesToStream += Stream->GetNumFramesNeeded();
		}

		// Now, schedule new read requests according to the number of concurrent reads available
		NumStreams = TracksToStreams.Num();
		int32 AvailableReads = MaxReads - NumReads;
		if (NumStreams > 0 && AvailableReads > 0)
		{
			TArray<FRefCountedStream*> Streams;
			TracksToStreams.GenerateValueArray(Streams);

			// Streams are checked round robin until there are no more reads available
			// or no stream can handle more read requests
			// Note that the round robin starts from where it left off in the previous Tick
			TBitArray<> StreamsToCheck(true, NumStreams);
			for (; AvailableReads > 0 && StreamsToCheck.Contains(true); ++CurrentIndex)
			{
				// Handle looping but also the case where the number of streams has decreased
				if (CurrentIndex >= NumStreams)
				{
					CurrentIndex = 0;
				}

				if (!StreamsToCheck[CurrentIndex])
				{
					continue;
				}

				IGeometryCacheStream* Stream = Streams[CurrentIndex]->Stream;
				if (Stream->GetNumFramesNeeded() > 0)
				{
					if (Stream->RequestFrameData())
					{
						// Stream was able to handle the read request so there's one less available
						++NumReads;
						--AvailableReads;
					}
					else
					{
						// Stream cannot handle more read request, don't need to check it again
						StreamsToCheck[CurrentIndex] = false;
					}
				}
				else
				{
					// Stream doesn't need any frame to be read, don't need to check it again
					StreamsToCheck[CurrentIndex] = false;
				}
			}
		}

		// If this cvar is enabled, tick until all frames have finished streaming
		bWaitForFinish = GGeoCacheStreamerBlockTillFinishStreaming && (NumFramesToStream > 0);
		if (bWaitForFinish)
		{
			FPlatformProcess::Sleep(.1f);
		}
	} while (bWaitForFinish);

	INC_DWORD_STAT_BY(STAT_GeometryCacheStream_Count, NumStreams);

	// Display a streaming progress notification if the the number of frames to stream is above the threshold
	// to prevent spamming notifications when there's only a few frames to stream
	if (GShowGeometryCacheStreamerNotification && !StreamingNotification.IsValid() && NumFramesToStream >= 24)
	{
		FText UpdateText = FText::Format(LOCTEXT("GeoCacheStreamingUpdate", "Streaming GeometryCache: {0} frames remaining"), FText::AsNumber(NumFramesToStream));
		FNotificationInfo Info(UpdateText);
		Info.bFireAndForget = false;
		Info.bUseSuccessFailIcons = false;
		Info.bUseLargeFont = false;

		StreamingNotification = FSlateNotificationManager::Get().AddNotification(Info);
		if (StreamingNotification.IsValid())
		{
			StreamingNotification->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}

	if (StreamingNotification.IsValid())
	{
		// Update or remove the progress notification
		if (NumFramesToStream > 0)
		{
			FText UpdateText = FText::Format(LOCTEXT("GeoCacheStreamingUpdate", "Streaming GeometryCache: {0} frames remaining"), FText::AsNumber(NumFramesToStream));
			StreamingNotification->SetText(UpdateText);
		}
		else
		{
			FText CompletedText = LOCTEXT("GeoCacheStreamingFinished", "Finished streaming GeometryCache");
			StreamingNotification->SetText(CompletedText);
			StreamingNotification->SetCompletionState(SNotificationItem::CS_Success);
			StreamingNotification->ExpireAndFadeout();
			StreamingNotification = nullptr;
		}
	}
}

void FGeometryCacheStreamer::BalanceStreams()
{
	// Collect the stats for all streams
	FGeometryCacheStreamStats AggregatedStats;
	for (const auto& Pair : TracksToStreams)
	{
		IGeometryCacheStream* Stream = Pair.Value->Stream;
		const FGeometryCacheStreamStats& StreamStats = Stream->GetStreamStats();
		AggregatedStats.MemoryUsed += StreamStats.MemoryUsed;
		AggregatedStats.CachedDuration = FMath::Max(AggregatedStats.CachedDuration, StreamStats.CachedDuration);
		AggregatedStats.AverageBitrate += StreamStats.AverageBitrate;
	}

	SET_MEMORY_STAT(STAT_GeometryCacheStream_MemoryUsed, AggregatedStats.MemoryUsed * 1024 * 1024);
	SET_FLOAT_STAT(STAT_GeometryCacheStream_LookAhead, AggregatedStats.CachedDuration);
	SET_FLOAT_STAT(STAT_GeometryCacheStream_Bitrate, AggregatedStats.AverageBitrate);

	const UGeometryCacheStreamerSettings* Settings = GetDefault<UGeometryCacheStreamerSettings>();
	const float MemoryLimit = Settings->MaxMemoryAllowed;
	const float DurationLimit = Settings->LookAheadBuffer;

	// Apply memory limits if over budget
	// Re-balance when under budget only when it's idle as it can naturally go under if it is starving
	bool bIsOverBudget = AggregatedStats.MemoryUsed > MemoryLimit || AggregatedStats.CachedDuration > DurationLimit;
	bool bIsUnderBudget = AggregatedStats.MemoryUsed < MemoryLimit * 0.75f || AggregatedStats.CachedDuration < DurationLimit;
	if (bIsOverBudget || (bIsUnderBudget && NumReads == 0))
	{
		// Allocate per-stream budget based on the duration computed from the total bitrate and given memory limit
		const float AvgDuration = MemoryLimit / AggregatedStats.AverageBitrate;
		for (const auto& Pair : TracksToStreams)
		{
			IGeometryCacheStream* Stream = Pair.Value->Stream;
			const FGeometryCacheStreamStats& StreamStats = Stream->GetStreamStats();

			const float MaxDuration = FMath::Clamp(AvgDuration, 0.f, DurationLimit);
			const float MaxMem = AvgDuration * StreamStats.AverageBitrate;
			Stream->SetLimits(MaxMem, MaxDuration);
		}
	}
}

void FGeometryCacheStreamer::RegisterTrack(UGeometryCacheTrack* Track, IGeometryCacheStream* Stream)
{
	if (FRefCountedStream** RefCountStream = TracksToStreams.Find(Track))
	{
		(*RefCountStream)->AddRef();
	}
	else
	{
		TracksToStreams.Add(Track, new FRefCountedStream(Stream));
	}
}

void FGeometryCacheStreamer::UnregisterTrack(UGeometryCacheTrack* Track)
{
	if (FRefCountedStream** RefCountStream = TracksToStreams.Find(Track))
	{
		int32 NumRequests = ((*RefCountStream)->Stream)->CancelRequests();
		NumReads -= NumRequests;

		// The refcounted object deletes itself when the count reaches 0
		if ((*RefCountStream)->Release() == 0)
		{
			TracksToStreams.Remove(Track);
		}
	}
}

bool FGeometryCacheStreamer::IsTrackRegistered(UGeometryCacheTrack* Track) const
{
	return TracksToStreams.Contains(Track);
}

bool FGeometryCacheStreamer::TryGetFrameData(UGeometryCacheTrack* Track, int32 FrameIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (FRefCountedStream** RefCountStream = TracksToStreams.Find(Track))
	{
		if (((*RefCountStream)->Stream)->GetFrameData(FrameIndex, OutMeshData))
		{
			return true;
		}
	}
	return false;
}

IGeometryCacheStreamer& IGeometryCacheStreamer::Get()
{
	static FGeometryCacheStreamer Streamer;
	return Streamer;
}

#undef LOCTEXT_NAMESPACE
