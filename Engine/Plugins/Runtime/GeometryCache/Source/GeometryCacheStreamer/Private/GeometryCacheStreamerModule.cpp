// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FGeometryCacheStreamerModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FGeometryCacheStreamerModule, GeometryCacheStreamer);
