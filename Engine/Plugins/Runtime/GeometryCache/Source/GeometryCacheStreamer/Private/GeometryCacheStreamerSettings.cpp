// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheStreamerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheStreamerSettings)

	 
UGeometryCacheStreamerSettings::UGeometryCacheStreamerSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("Geometry Cache");

	LookAheadBuffer = 4.0f;
	MaxMemoryAllowed = 4096.0f;
}

