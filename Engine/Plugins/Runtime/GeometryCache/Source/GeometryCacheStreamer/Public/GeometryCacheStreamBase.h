// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "IGeometryCacheStream.h"
#include "HAL/CriticalSection.h"

#define UE_API GEOMETRYCACHESTREAMER_API

struct FGeometryCacheStreamReadRequest;

/* Details about the animation to be streamed */
struct FGeometryCacheStreamDetails
{
	int32 NumFrames = 0;
	float Duration = 0.f;
	float SecondsPerFrame = 1.f / 24.f;
	int32 StartFrameIndex = 0;
	int32 EndFrameIndex = 0;
};

/**
 * Base class for GeometryCache stream for use with the GeometryCacheStreamer
 * Besides implementing the basic functionalities expected of the stream,
 * it implements basic memory statistics and management for use by the streamer.
 * Derived classes need to implement a way to retrieve the mesh data for a frame
 * through GetMeshData.
 */
class FGeometryCacheStreamBase : public IGeometryCacheStream
{
public:
	UE_API FGeometryCacheStreamBase(int32 ReadConcurrency, FGeometryCacheStreamDetails&& Details);
	UE_API virtual ~FGeometryCacheStreamBase();

	//~ Begin IGeometryCacheStream Interface
	UE_API virtual void Prefetch(int32 StartFrameIndex, int32 NumFrames = 0) override;
	UE_API virtual uint32 GetNumFramesNeeded() override;
	UE_API virtual bool RequestFrameData() override;
	UE_API virtual void UpdateRequestStatus(TArray<int32>& OutFramesCompleted) override;
	UE_API virtual bool GetFrameData(int32 FrameIndex, FGeometryCacheMeshData& OutMeshData) override;
	UE_API virtual int32 CancelRequests() override;
	UE_API virtual const FGeometryCacheStreamStats& GetStreamStats() const override;
	UE_API virtual void SetLimits(float MaxMemoryAllowed, float MaxCachedDuration) override;
	//~ End IGeometryCacheStream Interface

	/* Updates the current position in the stream */
	UE_API void UpdateCurrentFrameIndex(int32 FrameIndex);

protected:
	/* Function called from main thread to prepare for GetMeshData */
	virtual void PrepareRead() {}

	/* Derived class must provide a way to get the mesh data for the given FrameIndex, called from worker threads */
	virtual void GetMeshData(int32 FrameIndex, int32 ReadConcurrencyIndex, FGeometryCacheMeshData& OutMeshData) = 0;

	UE_API void LoadFrameData(int32 FrameIndex);
	UE_API void UpdateFramesNeeded(int32 StartIndex, int32 NumFrames);
	UE_API void IncrementMemoryStat(const FGeometryCacheMeshData& MeshData);
	UE_API void DecrementMemoryStat(const FGeometryCacheMeshData& MeshData);

	TArray<int32> ReadIndices;
	TArray<FGeometryCacheStreamReadRequest*> ReadRequestsPool;

	TArray<int32> FramesNeeded;
	TArray<int32> FramesToBeCached;
	TArray<FGeometryCacheStreamReadRequest*> FramesRequested;

	using FFrameIndexToMeshData = TMap<int32, FGeometryCacheMeshData*>;
	FFrameIndexToMeshData FramesAvailable;
	FRWLock FramesAvailableLock;

	FGeometryCacheStreamDetails Details;
	mutable FGeometryCacheStreamStats Stats;
	int32 CurrentFrameIndex;
	int32 MaxCachedFrames;
	float MaxCachedDuration;
	float MaxMemAllowed;
	float MemoryUsed;

	std::atomic<bool> bCancellationRequested;
	bool bCacheNeedsUpdate;
};

#undef UE_API
