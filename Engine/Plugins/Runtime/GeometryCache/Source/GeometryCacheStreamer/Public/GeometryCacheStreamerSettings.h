// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "GeometryCacheStreamerSettings.generated.h"

#define UE_API GEOMETRYCACHESTREAMER_API

/** Settings for the GeometryCache streamer */
UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "Geometry Cache"))
class UGeometryCacheStreamerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	 
	UE_API UGeometryCacheStreamerSettings();

	/** The amount of animation (in seconds) to stream ahead of time (per stream) */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry Cache Streamer", meta = (DisplayName = "Look-Ahead Buffer (in seconds)", ClampMin = "0.01", ClampMax = "3600.0"))
	float LookAheadBuffer;

	/** The maximum total amount of streamed data allowed in memory (for all streams) */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry Cache Streamer", meta = (DisplayName = "Maximum Memory Allowed (in MB)", ClampMin = "1.0", ClampMax = "262144.0"))
	float MaxMemoryAllowed;
};

#undef UE_API
