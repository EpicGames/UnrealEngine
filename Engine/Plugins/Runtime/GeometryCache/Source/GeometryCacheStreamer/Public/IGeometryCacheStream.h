// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h" // IWYU pragma: keep

struct FGeometryCacheMeshData;
struct FGeometryCacheStreamStats;

/** Interface to stream GeometryCacheMeshData asynchronously from any source through the GeometryCacheStreamer */
class IGeometryCacheStream
{
public:
	virtual ~IGeometryCacheStream() = default;

	/** Prefetch NumFrames starting from the given StartFrameIndex. If no NumFrames given, prefetch the whole stream */
	virtual void Prefetch(int32 StartFrameIndex, int32 NumFrames = 0) = 0;

	/** Return the number of frame indices needed to be loaded */
	virtual uint32 GetNumFramesNeeded() = 0;
	
	/** Request a read of the next frame as determined by the stream. Return true if the request could be handled */
	virtual bool RequestFrameData() = 0;

	/** Update the status of the read requests currently in progress. Return the frame indices that were completed */
	virtual void UpdateRequestStatus(TArray<int32>& OutFramesCompleted) = 0;

	/* Get the MeshData at given FrameIndex without waiting for data to be ready
	 * Return true if MeshData could be retrieved
	 */
	virtual bool GetFrameData(int32 FrameIndex, FGeometryCacheMeshData& OutMeshData) = 0;

	/** Cancel the scheduled read requests. Return the number of requests that were canceled */
	virtual int32 CancelRequests() = 0;

	/** Return the memory usage and related stats for the stream */
	virtual const FGeometryCacheStreamStats& GetStreamStats() const = 0;

	/** Set the memory usage limits for the stream */
	virtual void SetLimits(float MaxMemoryAllowed, float MaxCachedDuration) = 0;
};

struct FGeometryCacheStreamStats
{
	uint32 NumCachedFrames = 0; // number of frames currently resident in memory
	float CachedDuration = 0.0f;// in seconds
	float MemoryUsed = 0.0f;	// in MB
	float AverageBitrate = 0.0f;// in MB/s
};
