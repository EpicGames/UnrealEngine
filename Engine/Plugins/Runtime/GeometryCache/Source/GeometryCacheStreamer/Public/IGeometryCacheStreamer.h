// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API GEOMETRYCACHESTREAMER_API

struct FGeometryCacheMeshData;
class IGeometryCacheStream;
class UGeometryCacheTrack;

/** Interface to the GeometryCache Streamer that streams data for registered GeometryCacheTracks */
class IGeometryCacheStreamer
{
public:
	static UE_API IGeometryCacheStreamer& Get();

	/** Register the given Track and its associated Stream with the Streamer. The Streamer takes ownership of the Stream */
	virtual void RegisterTrack(UGeometryCacheTrack* Track, IGeometryCacheStream* Stream) = 0;

	/** Unregister the given Track from the Streamer */
	virtual void UnregisterTrack(UGeometryCacheTrack* Track) = 0;

	/** Return true if the given Track is registered with the Streamer */
	virtual bool IsTrackRegistered(UGeometryCacheTrack* Track) const = 0;

	/* Get the MeshData for a given Track at given FrameIndex without waiting for data to be ready
	 * Return true if MeshData could be retrieved
	 */
	virtual bool TryGetFrameData(UGeometryCacheTrack* Track, int32 FrameIndex, FGeometryCacheMeshData& OutMeshData) = 0;
};

#undef UE_API
