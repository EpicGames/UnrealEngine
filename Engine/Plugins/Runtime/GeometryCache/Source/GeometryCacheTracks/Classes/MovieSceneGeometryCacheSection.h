// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "MovieSceneGeometryCacheSection.generated.h"

#define UE_API GEOMETRYCACHETRACKS_API

class UGeometryCache;
struct FQualifiedFrameTime;

USTRUCT(BlueprintType)
 struct FMovieSceneGeometryCacheParams
{
	GENERATED_BODY()

	UE_API FMovieSceneGeometryCacheParams();

	/** Gets the animation sequence length, not modified by play rate */
	 UE_API float GetSequenceLength() const;
	/** The animation this section plays */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometryCache", DisplayName = "Geometry Cache")
	TObjectPtr<UGeometryCache> GeometryCacheAsset;

	/** The offset for the first loop of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometryCache")
	FFrameNumber FirstLoopStartFrameOffset;

	/** The offset into the beginning of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometryCache")
	FFrameNumber StartFrameOffset;

	/** The offset into the end of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometryCache")
	FFrameNumber EndFrameOffset;

	/** The playback rate of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometryCache")
	float PlayRate;

	/** Reverse the playback of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation")
	uint32 bReverse : 1;

	UPROPERTY()
	float StartOffset_DEPRECATED;

	UPROPERTY()
	float EndOffset_DEPRECATED;

	UPROPERTY()
	FSoftObjectPath GeometryCache_DEPRECATED;
};

/**
 * Movie scene section that control geometry cache playback
 */
UCLASS(MinimalAPI)
class UMovieSceneGeometryCacheSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation", meta = (ShowOnlyInnerProperties))
	FMovieSceneGeometryCacheParams Params;

	/** Get Frame Time as Animation Time*/
	virtual float MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const;

protected:
	//~ UMovieSceneSection interface
	virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const override;
	virtual TOptional<FFrameTime> GetOffsetTime() const override;
	virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) override;
	virtual UObject* GetSourceObject() const override;

	/** ~UObject interface */
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

public:

	//~ UObject interface

#if WITH_EDITOR

	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
public:
	float PreviousPlayRate;

#endif

};

#undef UE_API
