// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class GeometryCacheTracks : ModuleRules
    {
        public GeometryCacheTracks(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "MovieScene",
                    "MovieSceneTracks",
                    "GeometryCache",

                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "AnimGraphRuntime",
                    "TimeManagement",
                }
            );


            PublicIncludePathModuleNames.Add("TargetPlatform");

            if (Target.bBuildEditor)
            {
                PublicIncludePathModuleNames.Add("GeometryCacheSequencer");
                DynamicallyLoadedModuleNames.Add("GeometryCacheSequencer");
                
                PrivateDependencyModuleNames.AddRange(
	                new string[]
	                {
		                "SlateCore",
		                "Slate",
	                }
                );
            }

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
