// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTracksModule.h"
#if WITH_EDITOR
#include "GeometryCacheSequencerModule.h"
#else
#include "GeometryCacheModule.h"
#endif // WITH_EDITOR

IMPLEMENT_MODULE(FGeometryCacheTracksModule, GeometryCacheTracks)

void FGeometryCacheTracksModule::StartupModule()
{
	LLM_SCOPE_BYTAG(GeometryCache);
#if WITH_EDITOR
	FGeometryCacheSequencerModule& Module = FModuleManager::LoadModuleChecked<FGeometryCacheSequencerModule>(TEXT("GeometryCacheSequencer"));
#endif
}

void FGeometryCacheTracksModule::ShutdownModule()
{
}
