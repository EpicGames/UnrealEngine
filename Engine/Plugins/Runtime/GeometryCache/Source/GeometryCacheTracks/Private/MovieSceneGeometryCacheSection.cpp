// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGeometryCacheSection.h"
#include "GeometryCache.h"
#include "MovieScene.h"
#include "GeometryCacheComponent.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneGeometryCacheTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGeometryCacheSection)

#define LOCTEXT_NAMESPACE "MovieSceneGeometryCacheSection"

namespace
{
	float GeometryCacheDeprecatedMagicNumber = TNumericLimits<float>::Lowest();
}

FMovieSceneGeometryCacheParams::FMovieSceneGeometryCacheParams()
{
	GeometryCacheAsset = nullptr;
	GeometryCache_DEPRECATED = nullptr;
	StartOffset_DEPRECATED = GeometryCacheDeprecatedMagicNumber;
	EndOffset_DEPRECATED = GeometryCacheDeprecatedMagicNumber;
	PlayRate = 1.f;
	bReverse = false;
}

UMovieSceneGeometryCacheSection::UMovieSceneGeometryCacheSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BlendType = EMovieSceneBlendType::Absolute;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR

	PreviousPlayRate = Params.PlayRate;

#endif
}

TOptional<FFrameTime> UMovieSceneGeometryCacheSection::GetOffsetTime() const
{
	return TOptional<FFrameTime>(Params.FirstLoopStartFrameOffset);
}

void UMovieSceneGeometryCacheSection::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	if (Params.StartFrameOffset.Value > 0)
	{
		FFrameNumber NewStartFrameOffset = ConvertFrameTime(FFrameTime(Params.StartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.StartFrameOffset = NewStartFrameOffset;
	}

	if (Params.EndFrameOffset.Value > 0)
	{
		FFrameNumber NewEndFrameOffset = ConvertFrameTime(FFrameTime(Params.EndFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.EndFrameOffset = NewEndFrameOffset;
	}
	if (Params.FirstLoopStartFrameOffset.Value > 0)
	{
		FFrameNumber NewFirstLoopStartFrameOffset = ConvertFrameTime(FFrameTime(Params.FirstLoopStartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.FirstLoopStartFrameOffset = NewFirstLoopStartFrameOffset;
	}
}

UObject* UMovieSceneGeometryCacheSection::GetSourceObject() const
{
	return Params.GeometryCacheAsset;
}

void UMovieSceneGeometryCacheSection::PostLoad()
{
	Super::PostLoad();

	UMovieScene* MovieSceneOuter = GetTypedOuter<UMovieScene>();

	if (!MovieSceneOuter)
	{
		return;
	}

	FFrameRate DisplayRate = MovieSceneOuter->GetDisplayRate();
	FFrameRate TickResolution = MovieSceneOuter->GetTickResolution();

	if (Params.StartOffset_DEPRECATED != GeometryCacheDeprecatedMagicNumber)
	{
		Params.StartFrameOffset = ConvertFrameTime(FFrameTime::FromDecimal(DisplayRate.AsDecimal() * Params.StartOffset_DEPRECATED), DisplayRate, TickResolution).FrameNumber;

		Params.StartOffset_DEPRECATED = GeometryCacheDeprecatedMagicNumber;
	}

	if (Params.EndOffset_DEPRECATED != GeometryCacheDeprecatedMagicNumber)
	{
		Params.EndFrameOffset = ConvertFrameTime(FFrameTime::FromDecimal(DisplayRate.AsDecimal() * Params.EndOffset_DEPRECATED), DisplayRate, TickResolution).FrameNumber;

		Params.EndOffset_DEPRECATED = GeometryCacheDeprecatedMagicNumber;
	}

	if (Params.GeometryCache_DEPRECATED.ResolveObject() != nullptr
		&& Params.GeometryCacheAsset == nullptr)
	{
		UGeometryCacheComponent *Comp = Cast<UGeometryCacheComponent>(Params.GeometryCache_DEPRECATED.ResolveObject());
		if (Comp)
		{
			Params.GeometryCacheAsset = (Comp->GetGeometryCache());
		}
	}
}

void UMovieSceneGeometryCacheSection::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	Super::Serialize(Ar);
}

TOptional<TRange<FFrameNumber> > UMovieSceneGeometryCacheSection::GetAutoSizeRange() const
{
	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameTime AnimationLength = Params.GetSequenceLength() * FrameRate;
	int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f);

	return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + IFrameNumber + 1);
}


void UMovieSceneGeometryCacheSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	SetFlags(RF_Transactional);

	if (TryModify())
	{
		if (bTrimLeft)
		{
			FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

			Params.FirstLoopStartFrameOffset = HasStartFrame() ? GetFirstLoopStartOffsetAtTrimTime(TrimTime, Params, GetInclusiveStartFrame(), FrameRate) : 0;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
	}
}

UMovieSceneSection* UMovieSceneGeometryCacheSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	const FFrameNumber InitialFirstLoopStartFrameOffset = Params.FirstLoopStartFrameOffset;

	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	const FFrameNumber NewOffset = HasStartFrame() ? GetFirstLoopStartOffsetAtTrimTime(SplitTime, Params, GetInclusiveStartFrame(), FrameRate) : 0;

	UMovieSceneSection* NewSection = Super::SplitSection(SplitTime, bDeleteKeys);
	if (NewSection != nullptr)
	{
		UMovieSceneGeometryCacheSection* NewGeometrySection = Cast<UMovieSceneGeometryCacheSection>(NewSection);
		NewGeometrySection->Params.FirstLoopStartFrameOffset = NewOffset;
	}

	// Restore original offset modified by splitting
	Params.FirstLoopStartFrameOffset = InitialFirstLoopStartFrameOffset;

	return NewSection;
}


void UMovieSceneGeometryCacheSection::GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
{
	Super::GetSnapTimes(OutSnapTimes, bGetSectionBorders);

	const FFrameRate   FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	const FFrameNumber StartFrame = GetInclusiveStartFrame();
	const FFrameNumber EndFrame = GetExclusiveEndFrame() - 1; // -1 because we don't need to add the end frame twice

	const float AnimPlayRate = FMath::IsNearlyZero(Params.PlayRate) ? 1.0f : Params.PlayRate;
	const float SeqLengthSeconds = static_cast<float>(Params.GetSequenceLength() - FrameRate.AsSeconds(Params.StartFrameOffset + Params.EndFrameOffset) / AnimPlayRate);
	const float FirstLoopSeqLengthInSeconds = static_cast<float>(SeqLengthSeconds - FrameRate.AsSeconds(Params.FirstLoopStartFrameOffset) / AnimPlayRate);

	const FFrameTime SequenceFrameLength = SeqLengthSeconds * FrameRate;
	const FFrameTime FirstLoopSequenceFrameLength = FirstLoopSeqLengthInSeconds * FrameRate;
	if (SequenceFrameLength.FrameNumber > 1)
	{
		// Snap to the repeat times
		bool IsFirstLoop = true;
		FFrameTime CurrentTime = StartFrame;
		while (CurrentTime < EndFrame)
		{
			OutSnapTimes.Add(CurrentTime.FrameNumber);
			if (IsFirstLoop)
			{
				CurrentTime += FirstLoopSequenceFrameLength;
				IsFirstLoop = false;
			}
			else
			{
				CurrentTime += SequenceFrameLength;
			}
		}
	}
}

float UMovieSceneGeometryCacheSection::MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const
{
	FMovieSceneGeometryCacheSectionTemplateParameters TemplateParams(Params, GetInclusiveStartFrame(), GetExclusiveEndFrame());
	return TemplateParams.MapTimeToAnimation(ComponentDuration, InPosition, InTickResolution);
}


#if WITH_EDITOR
void UMovieSceneGeometryCacheSection::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Store the current play rate so that we can compute the amount to compensate the section end time when the play rate changes
	PreviousPlayRate = Params.PlayRate;

	Super::PreEditChange(PropertyAboutToChange);
}

void UMovieSceneGeometryCacheSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Adjust the duration automatically if the play rate changes
	if (PropertyChangedEvent.Property != nullptr &&
		PropertyChangedEvent.Property->GetFName() == TEXT("PlayRate"))
	{
		float NewPlayRate = Params.PlayRate;

		if (!FMath::IsNearlyZero(NewPlayRate))
		{
			float CurrentDuration = static_cast<float>(UE::MovieScene::DiscreteSize(GetRange()));
			float NewDuration = CurrentDuration * (PreviousPlayRate / NewPlayRate);
			SetEndFrame(GetInclusiveStartFrame() + FMath::FloorToInt(NewDuration));

			PreviousPlayRate = NewPlayRate;
		}
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

float FMovieSceneGeometryCacheParams::GetSequenceLength() const
{
	return GeometryCacheAsset != nullptr ? GeometryCacheAsset->CalculateDuration() : 0.f;
}

#undef LOCTEXT_NAMESPACE 

