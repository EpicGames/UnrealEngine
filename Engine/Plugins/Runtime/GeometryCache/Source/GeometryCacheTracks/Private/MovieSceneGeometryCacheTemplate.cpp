// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGeometryCacheTemplate.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "GeometryCacheComponent.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGeometryCacheTemplate)


DECLARE_CYCLE_STAT(TEXT("Geometry Cache Evaluate"), MovieSceneEval_GeometryCache_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Geometry Cache Token Execute"), MovieSceneEval_GeometryCache_TokenExecute, STATGROUP_MovieSceneEval);

/** Used to set Manual Tick back to previous when outside section */
struct FPreAnimatedGeometryCacheTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			FToken(UGeometryCacheComponent* InComponent)
			{
				// Cache this object's current update flag and animation mode
				bInManualTick = InComponent->GetManualTick();
			}

			virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params)
			{
				UGeometryCacheComponent* Component = CastChecked<UGeometryCacheComponent>(&Object);
				Component->SetManualTick(bInManualTick);
				Component->ResetAnimationTime();
			}
			bool bInManualTick;
		};

		return FToken(CastChecked<UGeometryCacheComponent>(&Object));
	}
	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FPreAnimatedGeometryCacheTokenProducer>();
	}
};


/** A movie scene execution token that executes a geometry cache */
struct FGeometryCacheExecutionToken
	: IMovieSceneExecutionToken

{
	FGeometryCacheExecutionToken(const FMovieSceneGeometryCacheSectionTemplateParameters &InParams) :
		Params(InParams)
	{}

	static UGeometryCacheComponent* GeometryMeshComponentFromObject(UObject* BoundObject)
	{
		if (AActor* Actor = Cast<AActor>(BoundObject))
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UGeometryCacheComponent* GeometryMeshComp = Cast<UGeometryCacheComponent>(Component))
				{
					return GeometryMeshComp;
				}
			}
		}
		else if (UGeometryCacheComponent* GeometryMeshComp = Cast<UGeometryCacheComponent>(BoundObject))
		{
			if (GeometryMeshComp->GetGeometryCache())
			{
				return GeometryMeshComp;
			}
		}
		return nullptr;
	}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_GeometryCache_TokenExecute)
		if (Operand.ObjectBindingID.IsValid())
		{
			for (TWeakObjectPtr<> WeakObj : Player.FindBoundObjects(Operand))
			{
				if (UObject* Obj = WeakObj.Get())
				{
					UGeometryCacheComponent* GeometryComp = GeometryMeshComponentFromObject(Obj);
					if (GeometryComp && GeometryComp->IsRegistered())
					{
						// Set the GeometryCache on the component only if it's set and valid in the Params
						if (Params.GeometryCacheAsset && Params.GeometryCacheAsset != GeometryComp->GetGeometryCache())
						{
							UGeometryCache* GeomCache = Params.GeometryCacheAsset;
							{
								GeometryComp->SetGeometryCache(GeomCache);
							}
						}
						else
						{
							// It could be unset if the Params was referencing a transient GeometryCache
							// In that case, use the GeometryCache that is set on the component
							Params.GeometryCacheAsset = GeometryComp->GetGeometryCache();
						}
						Player.SavePreAnimatedState(*GeometryComp, FPreAnimatedGeometryCacheTokenProducer::GetAnimTypeID(), FPreAnimatedGeometryCacheTokenProducer());
						GeometryComp->SetManualTick(true);
						// calculate the time at which to evaluate the animation
						float EvalTime = Params.MapTimeToAnimation(GeometryComp->GetDuration(), Context.GetTime(), Context.GetFrameRate());
						GeometryComp->TickAtThisTime(EvalTime, true, Params.bReverse, true);
					}
				}
			}
		}
	}

	FMovieSceneGeometryCacheSectionTemplateParameters Params;
};

FMovieSceneGeometryCacheSectionTemplate::FMovieSceneGeometryCacheSectionTemplate(const UMovieSceneGeometryCacheSection& InSection)
	: Params(const_cast<FMovieSceneGeometryCacheParams&> (InSection.Params), InSection.GetInclusiveStartFrame(), InSection.GetExclusiveEndFrame())
{
}

//We use a token here so we can set the manual tick state back to what it was previously when outside this section.
//This is similar to how Skeletal Animation evaluation also works.
void FMovieSceneGeometryCacheSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_GeometryCache_Evaluate)
		ExecutionTokens.Add(FGeometryCacheExecutionToken(Params));
}

float FMovieSceneGeometryCacheSectionTemplateParameters::MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const
{
	const float SequenceLength = ComponentDuration;
	const FFrameTime AnimationLength = SequenceLength * InTickResolution;
	const int32 LengthInFrames = AnimationLength.FrameNumber.Value + (int)(AnimationLength.GetSubFrame() + 0.5f) + 1;
	//we only play end if we are not looping, and assuming we are looping if Length is greater than default length;
	const bool bLooping = (SectionEndTime.Value - SectionStartTime.Value + StartFrameOffset + EndFrameOffset) > LengthInFrames;

	InPosition = FMath::Clamp(InPosition, FFrameTime(SectionStartTime), FFrameTime(SectionEndTime - 1));

	const float SectionPlayRate = PlayRate;
	const float AnimPlayRate = FMath::IsNearlyZero(SectionPlayRate) ? 1.0f : SectionPlayRate;

	const float FirstLoopSeqLength = static_cast<float>(SequenceLength - InTickResolution.AsSeconds(FirstLoopStartFrameOffset + StartFrameOffset + EndFrameOffset));
	const float SeqLength = static_cast<float>(SequenceLength - InTickResolution.AsSeconds(StartFrameOffset + EndFrameOffset));

	float AnimPosition = static_cast<float>(FFrameTime::FromDecimal((InPosition - SectionStartTime).AsDecimal() * AnimPlayRate) / InTickResolution);
	AnimPosition += static_cast<float>(InTickResolution.AsSeconds(FirstLoopStartFrameOffset));
	if (SeqLength > 0.f && (bLooping || !FMath::IsNearlyEqual(AnimPosition, SeqLength, 1e-4f)))
	{
		AnimPosition = FMath::Fmod(AnimPosition, SeqLength);
	}
	AnimPosition += static_cast<float>(InTickResolution.AsSeconds(StartFrameOffset));
	if (bReverse)
	{
		AnimPosition = SequenceLength - AnimPosition;
	}

	return AnimPosition;
}

