// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneGeometryCacheSection.h"
#include "MovieSceneGeometryCacheTemplate.generated.h"

USTRUCT()
struct FMovieSceneGeometryCacheSectionTemplateParameters : public FMovieSceneGeometryCacheParams
{
	GENERATED_BODY()

		FMovieSceneGeometryCacheSectionTemplateParameters() {}
	FMovieSceneGeometryCacheSectionTemplateParameters(const FMovieSceneGeometryCacheParams& BaseParams, FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime)
		: FMovieSceneGeometryCacheParams(BaseParams)
		, SectionStartTime(InSectionStartTime)
		, SectionEndTime(InSectionEndTime)
	{}

	float MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const;

	UPROPERTY()
	FFrameNumber SectionStartTime;

	UPROPERTY()
	FFrameNumber SectionEndTime;
};

USTRUCT()
struct FMovieSceneGeometryCacheSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneGeometryCacheSectionTemplate() {}
	FMovieSceneGeometryCacheSectionTemplate(const UMovieSceneGeometryCacheSection& Section);

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneGeometryCacheSectionTemplateParameters Params;
};
