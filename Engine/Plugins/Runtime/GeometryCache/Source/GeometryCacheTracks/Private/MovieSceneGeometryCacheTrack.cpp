// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGeometryCacheTrack.h"
#include "GeometryCacheComponent.h"
#include "MovieSceneGeometryCacheTemplate.h"
#include "MovieSceneGeometryCacheTemplate.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGeometryCacheTrack)

#define LOCTEXT_NAMESPACE "MovieSceneGeometryCacheTrack"


/* UMovieSceneGeometryCacheTrack structors
 *****************************************************************************/

UMovieSceneGeometryCacheTrack::UMovieSceneGeometryCacheTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(124, 15, 124, 65);
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bCanEvaluateNearestSection = true;
	EvalOptions.bEvaluateInPreroll = true;
}


/* UMovieSceneGeometryCacheTrack interface
 *****************************************************************************/

UMovieSceneSection* UMovieSceneGeometryCacheTrack::AddNewAnimation(FFrameNumber KeyTime, UGeometryCacheComponent* GeomCacheComp)
{
	UMovieSceneGeometryCacheSection* NewSection = Cast<UMovieSceneGeometryCacheSection>(CreateNewSection());
	{
		FFrameTime AnimationLength = GeomCacheComp->GetDuration()* GetTypedOuter<UMovieScene>()->GetTickResolution();
		int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f) + 1;
		NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, IFrameNumber, INDEX_NONE);
		
		NewSection->Params.GeometryCacheAsset = (GeomCacheComp->GetGeometryCache());
	}

	AddSection(*NewSection);

	return NewSection;
}


TArray<UMovieSceneSection*> UMovieSceneGeometryCacheTrack::GetAnimSectionsAtTime(FFrameNumber Time)
{
	TArray<UMovieSceneSection*> Sections;
	for (auto Section : AnimationSections)
	{
		if (Section->IsTimeWithinSection(Time))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}

FMovieSceneEvalTemplatePtr UMovieSceneGeometryCacheTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneGeometryCacheSectionTemplate(*CastChecked<UMovieSceneGeometryCacheSection>(&InSection));
}

/* UMovieSceneTrack interface
 *****************************************************************************/


const TArray<UMovieSceneSection*>& UMovieSceneGeometryCacheTrack::GetAllSections() const
{
	return AnimationSections;
}


bool UMovieSceneGeometryCacheTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneGeometryCacheSection::StaticClass();
}

UMovieSceneSection* UMovieSceneGeometryCacheTrack::CreateNewSection()
{
	return NewObject<UMovieSceneGeometryCacheSection>(this, NAME_None, RF_Transactional);
}


void UMovieSceneGeometryCacheTrack::RemoveAllAnimationData()
{
	AnimationSections.Empty();
}


bool UMovieSceneGeometryCacheTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}


void UMovieSceneGeometryCacheTrack::AddSection(UMovieSceneSection& Section)
{
	AnimationSections.Add(&Section);
}


void UMovieSceneGeometryCacheTrack::RemoveSection(UMovieSceneSection& Section)
{
	AnimationSections.Remove(&Section);
}

void UMovieSceneGeometryCacheTrack::RemoveSectionAt(int32 SectionIndex)
{
	AnimationSections.RemoveAt(SectionIndex);
}


bool UMovieSceneGeometryCacheTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneGeometryCacheTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Geometry Cache");
}

#endif


#undef LOCTEXT_NAMESPACE

